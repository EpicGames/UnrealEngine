// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class LevelInstanceEditor : ModuleRules
{
    public LevelInstanceEditor(ReadOnlyTargetRules Target) : base(Target)
    {
        PrivateIncludePathModuleNames.AddRange(
            new string[] {
            }
        );
     
        PrivateDependencyModuleNames.AddRange(
            new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"InputCore",
				"Slate",
				"SlateCore",
				"EditorFramework",
				"UnrealEd",
				"LevelEditor",
				"ToolMenus",
				"PropertyEditor",
				"NewLevelDialog",
				"MainFrame",
				"ContentBrowser",
				"AssetTools",
				"ClassViewer",
				"MessageLog",
				"EditorWidgets",
				"DeveloperSettings",
				"SceneOutliner",
				"WorldPartitionEditor",
				"Kismet",
				"PropertyEditor",
				"InteractiveToolsFramework",
				"EditorInteractiveToolsFramework",
				"TypedElementFramework",
				"TypedElementRuntime",
				"SceneOutliner"
			}
		);

        DynamicallyLoadedModuleNames.AddRange(
            new string[] {
		    }
		);
    }
}
