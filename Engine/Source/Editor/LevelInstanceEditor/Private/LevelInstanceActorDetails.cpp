// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelInstanceActorDetails.h"
#include "LevelInstance/LevelInstanceSettings.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "DetailWidgetRow.h"
#include "UObject/WeakInterfacePtr.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"
#include "ScopedTransaction.h"

#include "Engine/World.h"
#include "LevelInstance/LevelInstanceInterface.h"

#define LOCTEXT_NAMESPACE "FLevelInstanceActorDetails"

struct FLevelInstanceActorDetailsHelper
{
	static void ResetPropertyOverrides(ILevelInstanceInterface* LevelInstance)
	{
		check(LevelInstance);
		LevelInstance->GetLevelInstanceSubsystem()->ResetPropertyOverrides(LevelInstance);
	}
};

namespace LevelInstanceActorDetailsCallbacks
{
	static bool IsEditButtonEnabled(TWeakInterfacePtr<ILevelInstanceInterface> LevelInstancePtr)
	{
		if (ILevelInstanceInterface* LevelInstance = LevelInstancePtr.Get())
		{
			return LevelInstance->CanEnterEdit();
		}

		return false;
	}

	static FText GetEditButtonTooltip(TWeakInterfacePtr<ILevelInstanceInterface> LevelInstancePtr)
	{
		FText Reason;
		if (ILevelInstanceInterface* LevelInstance = LevelInstancePtr.Get())
		{
			if (!LevelInstance->IsEditing())
			{
				if (!LevelInstance->CanEnterEdit(&Reason))
				{
					return Reason;
				}

				return LOCTEXT("EditButtonToolTip", "Edit level instance source level");
			}
		}
		return FText::GetEmpty();
	}

	static EVisibility GetEditButtonVisibility(TWeakInterfacePtr<ILevelInstanceInterface> LevelInstancePtr)
	{
		if (ILevelInstanceInterface* LevelInstance = LevelInstancePtr.Get())
		{
			return (!LevelInstance->IsEditing() && !LevelInstance->IsEditingPropertyOverrides()) ? EVisibility::Visible : EVisibility::Collapsed;
		}

		return EVisibility::Collapsed;
	}
		
	static FReply OnEditButtonClicked(TWeakInterfacePtr<ILevelInstanceInterface> LevelInstancePtr)
	{
		if (ILevelInstanceInterface* LevelInstance = LevelInstancePtr.Get())
		{
			if (LevelInstance->CanEnterEdit())
			{
				LevelInstance->EnterEdit();
			}
		}
		return FReply::Handled();
	}

	static bool IsOverrideButtonEnabled(TWeakInterfacePtr<ILevelInstanceInterface> LevelInstancePtr)
	{
		if (ILevelInstanceInterface* LevelInstance = LevelInstancePtr.Get())
		{
			return LevelInstance->CanEnterEditPropertyOverrides();
		}

		return false;
	}

	static EVisibility GetOverrideButtonVisibility(TWeakInterfacePtr<ILevelInstanceInterface> LevelInstancePtr)
	{
		if (ULevelInstanceSettings::Get()->IsPropertyOverrideEnabled())
		{
			return GetEditButtonVisibility(LevelInstancePtr);
		}

		return EVisibility::Collapsed;
	}

	static FText GetOverrideButtonTooltip(TWeakInterfacePtr<ILevelInstanceInterface> LevelInstancePtr)
	{
		FText Reason;
		if (ILevelInstanceInterface* LevelInstance = LevelInstancePtr.Get())
		{
			if (!LevelInstance->IsEditingPropertyOverrides())
			{
				if (!LevelInstance->CanEnterEditPropertyOverrides(&Reason))
				{
					return Reason;
				}

				return LOCTEXT("OverrideButtonToolTip", "Override properties on level instance actors");
			}
		}
		return FText::GetEmpty();
	}

	static FReply OnOverrideButtonClicked(TWeakInterfacePtr<ILevelInstanceInterface> LevelInstancePtr)
	{
		if (ILevelInstanceInterface* LevelInstance = LevelInstancePtr.Get())
		{
			if (LevelInstance->CanEnterEditPropertyOverrides())
			{
				LevelInstance->EnterEditPropertyOverrides();
			}
		}
		return FReply::Handled();
	}

	static bool IsResetOverridesButtonEnabled(TWeakInterfacePtr<ILevelInstanceInterface> LevelInstancePtr)
	{
		if (ILevelInstanceInterface* LevelInstance = LevelInstancePtr.Get())
		{
			return LevelInstance->GetPropertyOverrideAsset() != nullptr;
		}

		return false;
	}

	static FText GetResetOverridesButtonTooltip(TWeakInterfacePtr<ILevelInstanceInterface> LevelInstancePtr)
	{
		return LOCTEXT("ResetOverrideButtonToolTip", "Reset property overrides on level instance actor"); 
	}

	static FReply OnResetOverridesButtonClicked(TWeakInterfacePtr<ILevelInstanceInterface> LevelInstancePtr)
	{
		if (ILevelInstanceInterface* LevelInstance = LevelInstancePtr.Get())
		{
			FLevelInstanceActorDetailsHelper::ResetPropertyOverrides(LevelInstance);
		}

		return FReply::Handled();
	}

	static EVisibility GetResetOverridesButtonVisibility(TWeakInterfacePtr<ILevelInstanceInterface> LevelInstancePtr)
	{
		if (ILevelInstanceInterface* LevelInstance = LevelInstancePtr.Get())
		{
			AActor* Actor = CastChecked<AActor>(LevelInstance);
			return ULevelInstanceSettings::Get()->IsPropertyOverrideEnabled() && !LevelInstance->IsEditing() && !LevelInstance->IsEditingPropertyOverrides() && (!Actor->IsInLevelInstance() || Actor->IsInEditLevelInstance()) ? EVisibility::Visible : EVisibility::Collapsed;
		}

		return EVisibility::Collapsed;
	}

	static EVisibility GetSaveCancelButtonVisibility(TWeakInterfacePtr<ILevelInstanceInterface> LevelInstancePtr)
	{
		if (ILevelInstanceInterface* LevelInstance = LevelInstancePtr.Get())
		{
			return (LevelInstance->IsEditing() || LevelInstance->IsEditingPropertyOverrides()) ? EVisibility::Visible : EVisibility::Collapsed;
		}

		return EVisibility::Collapsed;
	}
		
	static bool IsSaveCancelButtonEnabled(TWeakInterfacePtr<ILevelInstanceInterface> LevelInstancePtr, bool bDiscard)
	{
		if (ILevelInstanceInterface* LevelInstance = LevelInstancePtr.Get())
		{
			if (LevelInstance->IsEditing())
			{
				return LevelInstance->CanExitEdit(bDiscard);
			}
			else if (LevelInstance->IsEditingPropertyOverrides())
			{
				return LevelInstance->CanExitEditPropertyOverrides(bDiscard);
			}
		}

		return false;
	}

	static FText GetSaveCancelButtonTooltip(TWeakInterfacePtr<ILevelInstanceInterface> LevelInstancePtr, bool bDiscard)
	{
		FText Reason;
		if (ILevelInstanceInterface* LevelInstance = LevelInstancePtr.Get())
		{
			if (LevelInstance->IsEditing())
			{
				if (!LevelInstance->CanExitEdit(bDiscard, &Reason))
				{
					return Reason;
				}

				return bDiscard ? LOCTEXT("CancelButtonToolTip", "Cancel edits and exit") : LOCTEXT("SaveButtonToolTip", "Save edits and exit");
			}
			else if (LevelInstance->IsEditingPropertyOverrides())
			{
				if (!LevelInstance->CanExitEditPropertyOverrides(bDiscard, &Reason))
				{
					return Reason;
				}

				return bDiscard ? LOCTEXT("CancelOverrideButtonToolTip", "Cancel overrides and exit") : LOCTEXT("SaveOverrideButtonToolTip", "Save overrides and exit");
			}
		}
		return FText::GetEmpty();
	}

	static FReply OnSaveCancelButtonClicked(TWeakInterfacePtr<ILevelInstanceInterface> LevelInstancePtr, bool bDiscard)
	{
		if (ILevelInstanceInterface* LevelInstance = LevelInstancePtr.Get())
		{
			if (LevelInstance->IsEditing())
			{
				LevelInstance->ExitEdit(bDiscard);
			}
			else if (LevelInstance->IsEditingPropertyOverrides())
			{
				LevelInstance->ExitEditPropertyOverrides(bDiscard);
			}
		}
		return FReply::Handled();
	}
}

FLevelInstanceActorDetails::FLevelInstanceActorDetails()
{
}

TSharedRef<IDetailCustomization> FLevelInstanceActorDetails::MakeInstance()
{
	return MakeShareable(new FLevelInstanceActorDetails);
}

void FLevelInstanceActorDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> EditingObjects;
	DetailBuilder.GetObjectsBeingCustomized(EditingObjects);

	if (EditingObjects.Num() > 1)
	{
		return;
	}

	TWeakInterfacePtr<ILevelInstanceInterface> LevelInstance = Cast<ILevelInstanceInterface>(EditingObjects[0].Get());
	UWorld* World = LevelInstance.GetObject()->GetWorld();

	if (!World)
	{
		return;
	}

	if (ULevelInstancePropertyOverrideAsset* PropertyOverride = LevelInstance->GetPropertyOverrideAsset())
	{
		DetailBuilder.HideProperty("WorldAsset");
	}

	IDetailCategoryBuilder& LevelInstanceEditingCategory = DetailBuilder.EditCategory("LevelInstanceEdit", LOCTEXT("LevelInstanceEditCategory", "Level Instance"), ECategoryPriority::Transform);

	LevelInstanceEditingCategory.AddCustomRow(FText::GetEmpty()).ValueContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(4.0f, 8.0f, 4.0f, 8.0f)
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.IsEnabled_Static(&LevelInstanceActorDetailsCallbacks::IsEditButtonEnabled, LevelInstance)
				.ToolTipText_Static(&LevelInstanceActorDetailsCallbacks::GetEditButtonTooltip, LevelInstance)
				.Visibility_Static(&LevelInstanceActorDetailsCallbacks::GetEditButtonVisibility, LevelInstance)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.OnClicked_Static(&LevelInstanceActorDetailsCallbacks::OnEditButtonClicked, LevelInstance)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(LOCTEXT("EditText", "Edit"))
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(4.0f, 8.0f, 4.0f, 8.0f)
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.IsEnabled_Static(&LevelInstanceActorDetailsCallbacks::IsOverrideButtonEnabled, LevelInstance)
				.ToolTipText_Static(&LevelInstanceActorDetailsCallbacks::GetOverrideButtonTooltip, LevelInstance)
				.Visibility_Static(&LevelInstanceActorDetailsCallbacks::GetOverrideButtonVisibility, LevelInstance)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.OnClicked_Static(&LevelInstanceActorDetailsCallbacks::OnOverrideButtonClicked, LevelInstance)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(LOCTEXT("OverrideText", "Override"))
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(4.0f, 8.0f, 8.0f, 8.0f)
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.IsEnabled_Static(&LevelInstanceActorDetailsCallbacks::IsResetOverridesButtonEnabled, LevelInstance)
				.ToolTipText_Static(&LevelInstanceActorDetailsCallbacks::GetResetOverridesButtonTooltip, LevelInstance)
				.Visibility_Static(&LevelInstanceActorDetailsCallbacks::GetResetOverridesButtonVisibility, LevelInstance)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.OnClicked_Static(&LevelInstanceActorDetailsCallbacks::OnResetOverridesButtonClicked, LevelInstance)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(LOCTEXT("ResetOverrideText", "Reset Overrides"))
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(4.0f, 8.0f, 4.0f, 8.0f)
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "PrimaryButton")
				.IsEnabled_Static(&LevelInstanceActorDetailsCallbacks::IsSaveCancelButtonEnabled, LevelInstance, false)
				.ToolTipText_Static(&LevelInstanceActorDetailsCallbacks::GetSaveCancelButtonTooltip, LevelInstance, false)
				.Visibility_Static(&LevelInstanceActorDetailsCallbacks::GetSaveCancelButtonVisibility, LevelInstance)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.OnClicked_Static(&LevelInstanceActorDetailsCallbacks::OnSaveCancelButtonClicked, LevelInstance, false)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(LOCTEXT("SaveText", "Save"))
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(4.0f, 8.0f, 4.0f, 8.0f)
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.IsEnabled_Static(&LevelInstanceActorDetailsCallbacks::IsSaveCancelButtonEnabled, LevelInstance, true)
				.ToolTipText_Static(&LevelInstanceActorDetailsCallbacks::GetSaveCancelButtonTooltip, LevelInstance, true)
				.Visibility_Static(&LevelInstanceActorDetailsCallbacks::GetSaveCancelButtonVisibility, LevelInstance)
				.Text(LOCTEXT("DiscardText", "Discard"))
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.OnClicked_Static(&LevelInstanceActorDetailsCallbacks::OnSaveCancelButtonClicked, LevelInstance, true)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(LOCTEXT("CancelText", "Cancel"))
				]
			]
		];
}



#undef LOCTEXT_NAMESPACE