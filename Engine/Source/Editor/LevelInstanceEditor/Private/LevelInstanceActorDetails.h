// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Types/SlateEnums.h"
#include "IDetailCustomization.h"

class FLevelInstanceActorDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	/** End IDetailCustomization interface */

private:
	/** Use MakeInstance to create an instance of this class */
	FLevelInstanceActorDetails();
};