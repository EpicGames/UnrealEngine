// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelInstanceActorFactory.h"
#include "LevelInstanceEditorSettings.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "LevelInstance/LevelInstanceSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelInstanceActorFactory)

ULevelInstanceActorFactory::ULevelInstanceActorFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	NewActorClassName = GetDefault<ULevelInstanceEditorSettings>()->LevelInstanceClassName;
}

void ULevelInstanceActorFactory::PostSpawnActor(UObject* Asset, AActor* NewActor)
{
	ILevelInstanceInterface* LevelInstanceInterface = CastChecked<ILevelInstanceInterface>(NewActor);
	if (UWorld* WorldAsset = Cast<UWorld>(Asset))
	{
		LevelInstanceInterface->SetWorldAsset(WorldAsset);
		LevelInstanceInterface->LoadLevelInstance();
	}
}

bool ULevelInstanceActorFactory::CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg)
{
	if (GetDefault<ULevelInstanceSettings>()->IsLevelInstanceDisabled())
	{
		OutErrorMsg = NSLOCTEXT("LevelInstanceActorFactory", "LevelInstanceDisabled", "Level Instance support is disabled.");
		return false;
	}

	if (AssetData.IsValid())
	{
		// Only allow creating level instance actors from actor classes
		if (AssetData.IsInstanceOf<UClass>())
		{
			if (UClass* Class = Cast<UClass>(AssetData.GetAsset()))
			{
				if (!Class->IsChildOf<AActor>() || !Cast<ILevelInstanceInterface>(Class->GetDefaultObject()))
				{
					OutErrorMsg = NSLOCTEXT("LevelInstanceActorFactory", "InvalidClass", "A valid actor class must be specified.");
					return false;
				}
			
				return true;
			}
		}

		// Only allow creating level instance actors from world assets
		if (!AssetData.IsInstanceOf<UWorld>())
		{
			OutErrorMsg = NSLOCTEXT("LevelInstanceActorFactory", "NoWorld", "A valid world must be specified.");
			return false;
		}
	}

	return true;
}
