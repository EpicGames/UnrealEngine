// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ActorFactories/ActorFactory.h"
#include "LevelInstanceActorFactory.generated.h"

UCLASS(MinimalAPI, config = Editor)
class ULevelInstanceActorFactory : public UActorFactory
{
	GENERATED_UCLASS_BODY()

	//~ Begin UActorFactory Interface
	virtual void PostSpawnActor(UObject* Asset, AActor* NewActor) override;
	virtual bool CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg) override;
	//~ End UActorFactory Interface
};
