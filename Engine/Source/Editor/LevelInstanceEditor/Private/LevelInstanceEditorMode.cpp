// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelInstanceEditorMode.h"
#include "LevelInstanceEditorModeToolkit.h"
#include "LevelInstanceEditorModeCommands.h"
#include "LevelInstanceEditorSettings.h"
#include "Editor.h"
#include "Selection.h"
#include "EditorModes.h"
#include "Engine/World.h"
#include "EngineUtils.h"
#include "LevelInstance/LevelInstanceSettings.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "LevelInstance/ILevelInstanceEditorModule.h"
#include "LevelEditor.h"
#include "LevelEditorViewport.h"
#include "LevelEditorActions.h"
#include "EditorModeManager.h"
#include "Framework/Application/SlateApplication.h"
#include "Modules/ModuleManager.h"
#include "InteractiveToolManager.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "BaseBehaviors/MouseWheelBehavior.h"
#include "InputRouter.h"
#include "ToolContextInterfaces.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "Elements/Framework/TypedElementHandle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelInstanceEditorMode)

#define LOCTEXT_NAMESPACE "LevelInstanceEditorMode"

FEditorModeID ULevelInstanceEditorMode::EM_LevelInstanceEditorModeId("EditMode.LevelInstance");

class FMouseWheelBehaviorTarget : public IMouseWheelBehaviorTarget
{
public:
	FMouseWheelBehaviorTarget(UEditorInteractiveToolsContext* InInteractiveToolContext) : InteractiveToolContext(InInteractiveToolContext) {}

	UEditorInteractiveToolsContext* InteractiveToolContext = nullptr;

	// IMouseWheelBehaviorTarget
	virtual FInputRayHit ShouldRespondToMouseWheel(const FInputDeviceRay& CurrentPos) override
	{
		FInputRayHit ToReturn;

		TArray<AActor*> SelectionHierarchy;
		int32 SelectionIndex = INDEX_NONE;
		ToReturn.bHit = GetLevelInstanceSelectionHierarchy(CurrentPos, SelectionHierarchy, SelectionIndex);

		return ToReturn;
	}
	virtual void OnMouseWheelScrollUp(const FInputDeviceRay& CurrentPos) override
	{
		TArray<AActor*> SelectionHierarchy;
		int32 SelectionIndex = INDEX_NONE;
		if (GetLevelInstanceSelectionHierarchy(CurrentPos, SelectionHierarchy, SelectionIndex))
		{
			SelectActorAt(SelectionIndex - 1, SelectionHierarchy);
		}
	}

	virtual void OnMouseWheelScrollDown(const FInputDeviceRay& CurrentPos) override
	{
		TArray<AActor*> SelectionHierarchy;
		int32 SelectionIndex = INDEX_NONE;
		if (GetLevelInstanceSelectionHierarchy(CurrentPos, SelectionHierarchy, SelectionIndex))
		{
			SelectActorAt(SelectionIndex + 1, SelectionHierarchy);
		}
	}

private:
	bool GetLevelInstanceSelectionHierarchy(const FInputDeviceRay& CurrentPos, TArray<AActor*>& OutSelectionHierarchy, int32& OutSelectionIndex) const
	{
		if (!GetDefault<ULevelInstanceEditorPerProjectUserSettings>()->bIsViewportSubSelectionEnabled)
		{
			return false;
		}

		if (!FSlateApplication::Get().GetModifierKeys().IsShiftDown())
		{
			return false;
		}

		TArray<AActor*> SelectedActors;
		GEditor->GetSelectedActors()->GetSelectedObjects<AActor>(SelectedActors);

		// Only handle mouse wheel on single selection
		if (SelectedActors.Num() != 1)
		{
			return false;
		}

		AActor* SelectedActor = SelectedActors[0];

		if (CurrentPos.bHas2D)
		{
			if (IToolsContextQueriesAPI* ContextAPI = InteractiveToolContext->ToolManager->GetContextQueriesAPI())
			{
				if (FViewport* Viewport = ContextAPI->GetFocusedViewport())
				{
					if (HHitProxy* HitResult = Viewport->GetHitProxy(CurrentPos.ScreenPosition.X, CurrentPos.ScreenPosition.Y))
					{
						if (HActor* HitActor = HitProxyCast<HActor>(HitResult))
						{
							if (AActor* Actor = HitActor->Actor; Actor && Actor->IsInLevelInstance())
							{
								OutSelectionIndex = Actor == SelectedActor ? 0 : INDEX_NONE;
								OutSelectionHierarchy.Add(Actor);

								ULevelInstanceSubsystem* LevelInstanceSubsystem = UWorld::GetSubsystem<ULevelInstanceSubsystem>(Actor->GetWorld());
								LevelInstanceSubsystem->ForEachLevelInstanceAncestors(Actor, [SelectedActor, &OutSelectionIndex, &OutSelectionHierarchy](ILevelInstanceInterface* LevelInstanceInterface)
									{
										if (AActor* LevelInstanceActor = Cast<AActor>(LevelInstanceInterface))
										{
											OutSelectionIndex = LevelInstanceActor == SelectedActor ? OutSelectionHierarchy.Num() : OutSelectionIndex;
											OutSelectionHierarchy.Add(LevelInstanceActor);
										}
										return true;
									});

								return OutSelectionIndex != INDEX_NONE && OutSelectionHierarchy.Num() > 1;
							}
						}
					}
				}
			}
		}

		return false;
	}

	void SelectActorAt(int32 SelectionIndex, const TArray<AActor*>& SelectionHierarchy)
	{
		if (SelectionHierarchy.IsValidIndex(SelectionIndex))
		{
			AActor* ActorToSelect = SelectionHierarchy[SelectionIndex];
			if (ActorToSelect->SupportsSubRootSelection())
			{
				if (UTypedElementSelectionSet* SelectionSet = GEditor->GetSelectedActors()->GetElementSelectionSet())
				{
					const FTypedElementSelectionOptions SelectionOptions = FTypedElementSelectionOptions()
						.SetAllowHidden(true)
						.SetWarnIfLocked(false)
						.SetAllowLegacyNotifications(false)
						.SetAllowSubRootSelection(true);

					FTypedElementHandle ActorElementHandle = UEngineElementsLibrary::AcquireEditorActorElementHandle(SelectionHierarchy[SelectionIndex]);
					if(SelectionSet->CanSelectElement(ActorElementHandle, SelectionOptions))
					{
						SelectionSet->SetSelection(MakeArrayView(&ActorElementHandle,1), SelectionOptions);
					}
				}
			}
		}
	}
};

ULevelInstanceEditorMode::ULevelInstanceEditorMode()
	: UEdMode()
{
	Info = FEditorModeInfo(ULevelInstanceEditorMode::EM_LevelInstanceEditorModeId,
		LOCTEXT("LevelInstanceEditorModeName", "LevelInstanceEditorMode"),
		FSlateIcon(),
		false);

	bContextRestriction = true;
}

ULevelInstanceEditorMode::~ULevelInstanceEditorMode()
{
}

void ULevelInstanceEditorBehaviorSource::Initialize(UEditorInteractiveToolsContext* InteractiveToolsContext)
{
	InputBehaviorSet = NewObject<UInputBehaviorSet>();
	UMouseWheelInputBehavior* MouseWheelInputBehavior = NewObject<UMouseWheelInputBehavior>();
	MouseWheelBehaviorTarget = MakeUnique<FMouseWheelBehaviorTarget>(InteractiveToolsContext);
	MouseWheelInputBehavior->Initialize(MouseWheelBehaviorTarget.Get());
	MouseWheelInputBehavior->ModifierCheckFunc = [](const FInputDeviceState&)
	{
		return GetDefault<ULevelInstanceEditorPerProjectUserSettings>()->bIsViewportSubSelectionEnabled;
	};
	InputBehaviorSet->Add(MouseWheelInputBehavior);
}

TScriptInterface<IInputBehaviorSource> ULevelInstanceEditorMode::CreateDefaultModeBehaviorSource(UEditorInteractiveToolsContext* InteractiveToolContext)
{
	ULevelInstanceEditorBehaviorSource* NewBehaviorSource = NewObject<ULevelInstanceEditorBehaviorSource>();
	NewBehaviorSource->Initialize(InteractiveToolContext);
	return NewBehaviorSource;
}

void ULevelInstanceEditorMode::OnPreBeginPIE(bool bSimulate)
{
	ExitModeCommand();
}

void ULevelInstanceEditorMode::UpdateEngineShowFlags()
{
	for (FLevelEditorViewportClient* LevelVC : GEditor->GetLevelViewportClients())
	{
		if (LevelVC && LevelVC->GetWorld())
		{
			if(ULevelInstanceSubsystem* LevelInstanceSubsystem = LevelVC->GetWorld()->GetSubsystem<ULevelInstanceSubsystem>())
			{
				const bool bEditingLevelInstance = IsContextRestrictedForWorld(LevelVC->GetWorld());
				// Make sure we update both Game/Editor showflags
				LevelVC->EngineShowFlags.EditingLevelInstance = bEditingLevelInstance;
				LevelVC->LastEngineShowFlags.EditingLevelInstance = bEditingLevelInstance;
			}
		}
	}
}

void ULevelInstanceEditorMode::Enter()
{
	UEdMode::Enter();

	UpdateEngineShowFlags();
	
	if (UEditorInteractiveToolsContext* InteractiveToolContext = GetInteractiveToolsContext(EToolsContextScope::EdMode))
	{
		// UEdMode::Exit() can be deferred to on Tick which can cause potentially out of order Enter/Exit calls.
		// In the event that this does happen, we reregister the ModeBehaviorSource to prevent crashes, but ensure
		// because the subsequent Exit will deregister the newly reregistered source and break viewport sub selection.
		if (!ensureMsgf(ModeBehaviorSource == nullptr, TEXT("ModeBehaviorSource is already registered. Re-registering a new behavior source.")))
		{
			InteractiveToolContext->InputRouter->DeregisterSource(ModeBehaviorSource.GetInterface());
			ModeBehaviorSource = nullptr;
		}
		
		// Here we create a BehaviorSource specific to the Level Instance Editor Mode, for now it is the same type as the default one.
		ModeBehaviorSource = CreateDefaultModeBehaviorSource(InteractiveToolContext);
		InteractiveToolContext->InputRouter->RegisterSource(ModeBehaviorSource.GetInterface());
	}

	FEditorDelegates::PreBeginPIE.AddUObject(this, &ULevelInstanceEditorMode::OnPreBeginPIE);
}

void ULevelInstanceEditorMode::Exit()
{
	if (UEditorInteractiveToolsContext* InteractiveToolContext = GetInteractiveToolsContext(EToolsContextScope::EdMode))
	{
		InteractiveToolContext->InputRouter->DeregisterSource(ModeBehaviorSource.GetInterface());
		ModeBehaviorSource = nullptr;
	}

	UEdMode::Exit();
		
	UpdateEngineShowFlags();

	bContextRestriction = true;

	FEditorDelegates::PreBeginPIE.RemoveAll(this);
}

void ULevelInstanceEditorMode::CreateToolkit()
{
	Toolkit = MakeShared<FLevelInstanceEditorModeToolkit>();
}

void ULevelInstanceEditorMode::ModeTick(float DeltaTime)
{
	Super::ModeTick(DeltaTime);

	UpdateEngineShowFlags();
}

bool ULevelInstanceEditorMode::IsCompatibleWith(FEditorModeID OtherModeID) const
{
	return (OtherModeID != FBuiltinEditorModes::EM_Foliage) && ((OtherModeID != FBuiltinEditorModes::EM_Landscape) || ULevelInstanceSettings::Get()->IsLevelInstanceEditCompatibleWithLandscapeEdit());
}

void ULevelInstanceEditorMode::BindCommands()
{
	UEdMode::BindCommands();
	const TSharedRef<FUICommandList>& CommandList = Toolkit->GetToolkitCommands();
	const FLevelInstanceEditorModeCommands& Commands = FLevelInstanceEditorModeCommands::Get();

	CommandList->MapAction(
		Commands.ExitMode,
		FExecuteAction::CreateUObject(this, &ULevelInstanceEditorMode::ExitModeCommand),
		FCanExecuteAction::CreateLambda([&] 
		{ 
			// If some actors are selected make sure we don't interfere with the SelectNone command
			if(GEditor->GetSelectedActors()->Num() > 0)
			{
				const FInputChord& SelectNonePrimary = FLevelEditorCommands::Get().SelectNone->GetActiveChord(EMultipleKeyBindingIndex::Primary).Get();
				if (SelectNonePrimary.IsValidChord() && Commands.ExitMode->HasActiveChord(SelectNonePrimary))
				{
					return false;
				}

				const FInputChord& SelectNoneSecondary = FLevelEditorCommands::Get().SelectNone->GetActiveChord(EMultipleKeyBindingIndex::Secondary).Get();
				if (SelectNoneSecondary.IsValidChord() && Commands.ExitMode->HasActiveChord(SelectNoneSecondary))
				{
					return false;
				}
			}

			return true;
		}));

	CommandList->MapAction(
		Commands.ToggleContextRestriction,
		FExecuteAction::CreateUObject(this, &ULevelInstanceEditorMode::ToggleContextRestrictionCommand),
		FCanExecuteAction(),
		FIsActionChecked::CreateUObject(this, &ULevelInstanceEditorMode::IsContextRestrictionCommandEnabled));
}

bool ULevelInstanceEditorMode::IsEditingDisallowed(AActor* InActor) const
{
	return IsSelectionDisallowed(InActor, true);
}

bool ULevelInstanceEditorMode::IsSelectionDisallowed(AActor* InActor, bool bInSelection) const
{
	UWorld* World = InActor->GetWorld();
	const bool bRestrict = bInSelection && IsContextRestrictedForWorld(World);

	if (bRestrict)
	{
		check(World);
		if (ULevelInstanceSubsystem* LevelInstanceSubsystem = UWorld::GetSubsystem<ULevelInstanceSubsystem>(World))
		{
			const ILevelInstanceInterface* PropertyOverrideLevelInstance = LevelInstanceSubsystem->GetEditingPropertyOverridesLevelInstance();
			const ILevelInstanceInterface* EditLevelInstance = LevelInstanceSubsystem->GetEditingLevelInstance();

			if (ILevelInstanceInterface* LevelInstance = Cast<ILevelInstanceInterface>(InActor))
			{
				// If Actor is itself a Level Instance and is one of the edits, allow selection
				if (LevelInstance == PropertyOverrideLevelInstance || LevelInstance == EditLevelInstance)
				{
					return false;
				}
			}

			const ILevelInstanceInterface* ParentLevelInstance = LevelInstanceSubsystem->GetParentLevelInstance(InActor);
						
			auto IsAncestorOrSelf = [LevelInstanceSubsystem](const ILevelInstanceInterface* LevelInstance, const ILevelInstanceInterface* Ancestor)
			{
				while (LevelInstance != nullptr)
				{
					if (LevelInstance == Ancestor)
					{
						return true;
					}

					LevelInstance = LevelInstanceSubsystem->GetParentLevelInstance(CastChecked<AActor>(LevelInstance));
				}

				return false;
			};

			// If we have a PropertyOverride Edit in progress, actor can be selected if it is part of the PropertyOverrides hierarchy
			if (PropertyOverrideLevelInstance)
			{
				return !IsAncestorOrSelf(ParentLevelInstance, PropertyOverrideLevelInstance);
			}

			// If we have a Edit in progress, actor can be selected if it is part of the Edit hierarchy
			if (EditLevelInstance)
			{
				return !IsAncestorOrSelf(ParentLevelInstance, EditLevelInstance);
			}

			return false;
		}
	}

	return bRestrict;
}

void ULevelInstanceEditorMode::ExitModeCommand()
{	
	// Ignore command when any modal window is open
	if (FSlateApplication::IsInitialized() && FSlateApplication::Get().GetActiveModalWindow().IsValid())
	{
		return;
	}

	if (ILevelInstanceEditorModule* EditorModule = FModuleManager::GetModulePtr<ILevelInstanceEditorModule>("LevelInstanceEditor"))
	{
		EditorModule->BroadcastTryExitEditorMode();
	}
}

void ULevelInstanceEditorMode::ToggleContextRestrictionCommand()
{
	bContextRestriction = !bContextRestriction;

	UpdateEngineShowFlags();

	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	if (TSharedPtr<ILevelEditor> FirstLevelEditor = LevelEditorModule.GetFirstLevelEditor())
	{
		FirstLevelEditor->GetEditorModeManager().BroadcastIsEditingDisallowedChanged();
	}
}

bool ULevelInstanceEditorMode::IsContextRestrictionCommandEnabled() const
{
	return bContextRestriction;
}

bool ULevelInstanceEditorMode::IsContextRestrictedForWorld(UWorld* InWorld) const
{
	if (ULevelInstanceSubsystem* LevelInstanceSubsystem = InWorld? InWorld->GetSubsystem<ULevelInstanceSubsystem>() : nullptr)
	{
		if (ILevelInstanceInterface* EditingPropertyOverrides = LevelInstanceSubsystem->GetEditingPropertyOverridesLevelInstance())
		{
			// Always restrict outside selection while editing property overrides
			return true;
		}
		else if (ILevelInstanceInterface* EditingLevelInstance = LevelInstanceSubsystem->GetEditingLevelInstance())
		{
			return bContextRestriction && LevelInstanceSubsystem->GetLevelInstanceLevel(EditingLevelInstance) == InWorld->GetCurrentLevel();
		}
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
