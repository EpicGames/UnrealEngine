// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Tools/UEdMode.h"
#include "Framework/Commands/UICommandList.h"
#include "InputBehaviorSet.h"
#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "LevelInstanceEditorMode.generated.h"

UCLASS()
class ULevelInstanceEditorBehaviorSource : public UObject, public IInputBehaviorSource
{
	GENERATED_BODY()
public:
	virtual const UInputBehaviorSet* GetInputBehaviors() const override { return InputBehaviorSet; }
	void Initialize(UEditorInteractiveToolsContext* InteractiveToolsContext);

private:
	UPROPERTY(Transient)
	TObjectPtr<UInputBehaviorSet> InputBehaviorSet;

	TUniquePtr<IMouseWheelBehaviorTarget> MouseWheelBehaviorTarget;
};

UCLASS()
class ULevelInstanceEditorMode : public UEdMode
{
	GENERATED_BODY()
public:
	static FEditorModeID EM_LevelInstanceEditorModeId;

	/** Constructor */
	ULevelInstanceEditorMode();

	/** Destructor */
	virtual ~ULevelInstanceEditorMode();

	// Begin UEdMode
	virtual void Enter() override;
	virtual void Exit() override;
	virtual void CreateToolkit() override;
	virtual void ModeTick(float DeltaTime) override;
	
	virtual bool IsCompatibleWith(FEditorModeID OtherModeID) const override;

	/** Check to see if an actor can be selected in this mode - no side effects */
	virtual bool IsSelectionDisallowed(AActor* InActor, bool bInSelection) const override;
	virtual bool IsEditingDisallowed(AActor* InActor) const override;
	/** Only accept saving on current asset */
	virtual bool IsOperationSupportedForCurrentAsset(EAssetOperation InOperation) const { return InOperation == EAssetOperation::Save; }
	// End UEdMode

	static TScriptInterface<IInputBehaviorSource> CreateDefaultModeBehaviorSource(UEditorInteractiveToolsContext* InteractiveToolContext);
	
private:
	void OnPreBeginPIE(bool bSimulate);
	void UpdateEngineShowFlags();
	virtual void BindCommands() override;

	void ExitModeCommand();
	void ToggleContextRestrictionCommand();
	bool IsContextRestrictionCommandEnabled() const;
	bool IsContextRestrictedForWorld(UWorld* InWorld) const;

	bool bContextRestriction;
	
	UPROPERTY(Transient)
	TScriptInterface<IInputBehaviorSource> ModeBehaviorSource;
};

