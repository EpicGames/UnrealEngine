// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelInstanceEditorModeCommands.h"

#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "GenericPlatform/GenericApplication.h"
#include "InputCoreTypes.h"

#define LOCTEXT_NAMESPACE "LevelInstanceEditorModeCommands"

void FLevelInstanceEditorModeCommands::RegisterCommands()
{
	UI_COMMAND(ExitMode, "Exit Mode", "Exits mode asking to save pending changes", EUserInterfaceActionType::Button, FInputChord(EKeys::Escape));
	UI_COMMAND(ToggleContextRestriction, "Toggle Context Restriction", "Toggles edit restrictions like selection to current level edit", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::L, EModifierKey::Shift));
}

#undef LOCTEXT_NAMESPACE