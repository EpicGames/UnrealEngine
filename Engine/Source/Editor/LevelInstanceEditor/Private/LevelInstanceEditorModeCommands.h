// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "Internationalization/Internationalization.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class FUICommandInfo;

class FLevelInstanceEditorModeCommands : public TCommands<FLevelInstanceEditorModeCommands>
{

public:
	FLevelInstanceEditorModeCommands() : TCommands<FLevelInstanceEditorModeCommands>
		(
			"LevelInstanceEditorMode",
			NSLOCTEXT("Contexts", "LevelInstanceEditor", "Level Instance Editor"),
			NAME_None,
			FAppStyle::GetAppStyleSetName() // Icon Style Set
		)
	{}


	/** Exit Editor Mode */
	TSharedPtr<FUICommandInfo> ExitMode;

	/** Toggle Restrict Selection */
	TSharedPtr<FUICommandInfo> ToggleContextRestriction;
		
	/**
	 * Initialize commands
	 */
	virtual void RegisterCommands() override;
};