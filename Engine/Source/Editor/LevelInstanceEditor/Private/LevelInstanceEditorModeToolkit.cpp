// Copyright Epic Games, Inc. All Rights Reserved.
#include "LevelInstanceEditorModeToolkit.h"
#include "LevelInstanceEditorMode.h"
#include "Internationalization/Internationalization.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Tools/UEdMode.h"
#include "Toolkits/IToolkitHost.h"
#include "Styling/SlateIconFinder.h"
#include "LevelInstance/LevelInstanceActor.h"
#include "LevelInstance/ILevelInstanceEditorModule.h"
#include "Engine/World.h"

class FAssetEditorModeUILayer;

#define LOCTEXT_NAMESPACE "LevelInstanceEditorModeToolkit"

struct FLevelInstanceEditorModeToolkitHelper
{
	static FText GetToolkitDisplayText(ULevelInstanceSubsystem* LevelInstanceSubsystem)
	{
		if (LevelInstanceSubsystem->GetEditingLevelInstance())
		{
			return LOCTEXT("LevelInstanceEditToolkitDisplayText", "Level Instance Edit");
		}
		else if(LevelInstanceSubsystem->GetEditingPropertyOverridesLevelInstance())
		{
			return LOCTEXT("LevelInstanceOverrideToolkitDisplayText", "Level Instance Override");
		} 

		return FText();
	}

	static FText GetToolkitSaveCancelButtonTooltipText(ULevelInstanceSubsystem* LevelInstanceSubsystem, bool bDiscard)
	{
		if (LevelInstanceSubsystem->GetEditingLevelInstance())
		{
			return bDiscard ? LOCTEXT("LevelInstanceCancelEditToolkitTooltip", "Cancel edits and exit") : LOCTEXT("LevelInstanceSaveEditToolkitTooltip", "Save edits and exit");
		}
		else if (LevelInstanceSubsystem->GetEditingPropertyOverridesLevelInstance())
		{
			return bDiscard ? LOCTEXT("LevelInstanceCancelOverrideToolkitTooltip", "Cancel overrides and exit") : LOCTEXT("LevelInstanceSaveOverrideToolkitTooltip", "Save overrides and exit");
		}

		return FText();
	}

	static FReply OnSaveCancelButtonClicked(ULevelInstanceSubsystem* LevelInstanceSubsystem, bool bDiscard)
	{
		if (ILevelInstanceInterface* LevelInstance = LevelInstanceSubsystem->GetEditingLevelInstance())
		{
			LevelInstance->ExitEdit(bDiscard);
		}
		else if (ILevelInstanceInterface* LevelInstanceOverride = LevelInstanceSubsystem->GetEditingPropertyOverridesLevelInstance())
		{
			LevelInstanceOverride->ExitEditPropertyOverrides(bDiscard);
		}

		return FReply::Handled();
	}

	static bool IsCancelButtonEnabled(ULevelInstanceSubsystem* LevelInstanceSubsystem)
	{
		if (ILevelInstanceInterface* LevelInstance = LevelInstanceSubsystem->GetEditingLevelInstance())
		{
			return LevelInstance->CanExitEdit(true);
		}
		else if (ILevelInstanceInterface* LevelInstanceOverride = LevelInstanceSubsystem->GetEditingPropertyOverridesLevelInstance())
		{
			return LevelInstanceOverride->CanExitEditPropertyOverrides(true);
		}

		return false;
	}
};


FLevelInstanceEditorModeToolkit::FLevelInstanceEditorModeToolkit()
{
}

FLevelInstanceEditorModeToolkit::~FLevelInstanceEditorModeToolkit()
{
	if(IsHosted() && ViewportOverlayWidget.IsValid())
	{
		GetToolkitHost()->RemoveViewportOverlayWidget(ViewportOverlayWidget.ToSharedRef());
	}
}

void FLevelInstanceEditorModeToolkit::Init(const TSharedPtr<IToolkitHost>& InitToolkitHost, TWeakObjectPtr<UEdMode> InOwningMode)
{
	FModeToolkit::Init(InitToolkitHost, InOwningMode);

	ULevelInstanceSubsystem* LevelInstanceSubsystem = UWorld::GetSubsystem<ULevelInstanceSubsystem>(InitToolkitHost->GetWorld());
	check(LevelInstanceSubsystem);

	// ViewportOverlay
	SAssignNew(ViewportOverlayWidget, SHorizontalBox)
	+SHorizontalBox::Slot()
	.HAlign(HAlign_Center)
	.VAlign(VAlign_Bottom)
	.Padding(FMargin(0.0f, 0.0f, 0.f, 15.f))
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("EditorViewport.OverlayBrush"))
		.Padding(8.f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(4.f, 0.f, 0.f, 0.f))
			[
				SNew(SImage)
				.Image(FSlateIconFinder::FindIconBrushForClass(ALevelInstance::StaticClass()))
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(8.f, 0.f, 0.f, 0.f))
			[
				SNew(STextBlock)
				.Text_Static(&FLevelInstanceEditorModeToolkitHelper::GetToolkitDisplayText, LevelInstanceSubsystem)
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(8.0, 0.f, 0.f, 0.f))
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "PrimaryButton")
				.TextStyle(FAppStyle::Get(), "DialogButtonText")
				.Text(LOCTEXT("SaveButtonText", "Save"))
				.ToolTipText_Static(&FLevelInstanceEditorModeToolkitHelper::GetToolkitSaveCancelButtonTooltipText, LevelInstanceSubsystem, false)
				.HAlign(HAlign_Center)
				.OnClicked_Static(&FLevelInstanceEditorModeToolkitHelper::OnSaveCancelButtonClicked, LevelInstanceSubsystem, false)
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(4.0, 0.f, 4.f, 0.f))
			[
				SNew(SButton)
				.TextStyle(FAppStyle::Get(), "DialogButtonText")
				.Text(LOCTEXT("CancelButtonText", "Cancel"))
				.ToolTipText_Static(&FLevelInstanceEditorModeToolkitHelper::GetToolkitSaveCancelButtonTooltipText, LevelInstanceSubsystem, true)
				.HAlign(HAlign_Center)
				.OnClicked_Static(&FLevelInstanceEditorModeToolkitHelper::OnSaveCancelButtonClicked, LevelInstanceSubsystem, true)
				.IsEnabled_Static(&FLevelInstanceEditorModeToolkitHelper::IsCancelButtonEnabled, LevelInstanceSubsystem)
			]
		]
	];
	
	GetToolkitHost()->AddViewportOverlayWidget(ViewportOverlayWidget.ToSharedRef());
}

FName FLevelInstanceEditorModeToolkit::GetToolkitFName() const
{
	return FName("LevelInstanceEditorModeToolkit");
}

FText FLevelInstanceEditorModeToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("ToolkitDisplayName", "Level Instance Editor Mode");
}

void FLevelInstanceEditorModeToolkit::RequestModeUITabs()
{
	// No Tabs
}

#undef LOCTEXT_NAMESPACE