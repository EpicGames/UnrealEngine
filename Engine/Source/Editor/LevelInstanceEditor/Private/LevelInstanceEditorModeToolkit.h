// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "Toolkits/BaseToolkit.h"
#include "UObject/NameTypes.h"

class IToolkitHost;
class SWidget;
class UEdMode;

class FLevelInstanceEditorModeToolkit : public FModeToolkit
{
public:
	FLevelInstanceEditorModeToolkit();
	virtual ~FLevelInstanceEditorModeToolkit();

	// IToolkit interface 
	virtual void Init(const TSharedPtr<IToolkitHost>& InitToolkitHost, TWeakObjectPtr<UEdMode> InOwningMode);

	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;

	virtual void RequestModeUITabs() override;

private:
	TSharedPtr<SWidget> ViewportOverlayWidget;
};