// Copyright Epic Games, Inc. All Rights Reserved.
#include "LevelInstanceEditorModule.h"
#include "LevelInstanceActorDetails.h"
#include "LevelInstancePivotDetails.h"
#include "LevelInstanceSceneOutlinerColumn.h"
#include "PackedLevelActorUtils.h"
#include "LevelInstanceFilterPropertyTypeCustomization.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "LevelInstance/LevelInstanceActor.h"
#include "LevelInstance/LevelInstanceSettings.h"
#include "PackedLevelActor/PackedLevelActor.h"
#include "PackedLevelActor/PackedLevelActorBuilder.h"
#include "LevelInstanceEditorSettings.h"
#include "ToolMenus.h"
#include "Editor.h"
#include "EditorModeManager.h"
#include "EditorModeRegistry.h"
#include "FileHelpers.h"
#include "LevelInstanceEditorMode.h"
#include "LevelInstanceEditorModeCommands.h"
#include "LevelEditorMenuContext.h"
#include "ContentBrowserMenuContexts.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "LevelEditor.h"
#include "Engine/Selection.h"
#include "PropertyEditorModule.h"
#include "EditorLevelUtils.h"
#include "Modules/ModuleManager.h"
#include "Misc/MessageDialog.h"
#include "NewLevelDialogModule.h"
#include "Interfaces/IMainFrameModule.h"
#include "Editor/EditorEngine.h"
#include "AssetToolsModule.h"
#include "IAssetTools.h"
#include "Factories/BlueprintFactory.h"
#include "ClassViewerModule.h"
#include "ClassViewerFilter.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Misc/ScopeExit.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SWindow.h"
#include "SNewLevelInstanceDialog.h"
#include "MessageLogModule.h"
#include "Settings/EditorExperimentalSettings.h"
#include "WorldPartition/WorldPartitionConverter.h"
#include "WorldPartition/WorldPartitionActorLoaderInterface.h"
#include "ScopedTransaction.h"
#include "ISCSEditorUICustomization.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "SceneOutlinerModule.h"
#include "SceneOutlinerFwd.h"
#include "Subsystems/BrowseToAssetOverrideSubsystem.h"
#include "SceneOutlinerPublicTypes.h"

IMPLEMENT_MODULE( FLevelInstanceEditorModule, LevelInstanceEditor );

#define LOCTEXT_NAMESPACE "LevelInstanceEditor"

DEFINE_LOG_CATEGORY_STATIC(LogLevelInstanceEditor, Log, All);

struct FLevelInstanceMenuUtils
{
	static FToolMenuSection& CreateLevelSection(UToolMenu* Menu)
	{
		return CreateSection(Menu, FName("Level"), LOCTEXT("LevelSectionLabel", "Level"));
	}
	
	static FToolMenuSection& CreateCurrentEditSection(UToolMenu* Menu)
	{
		return CreateSection(Menu, FName("CurrentEdit"), LOCTEXT("CurrentEditSectionLabel", "Current Edit"));
	}

	static FToolMenuSection& CreateSection(UToolMenu* Menu, FName SectionName, const FText& SectionText)
	{
		FToolMenuSection* SectionPtr = Menu->FindSection(SectionName);
		if (!SectionPtr)
		{
			SectionPtr = &(Menu->AddSection(SectionName, SectionText));
		}
		FToolMenuSection& Section = *SectionPtr;
		return Section;
	}

	static void CreateEditMenuEntry(FToolMenuSection& Section, ILevelInstanceInterface* LevelInstance, AActor* ContextActor, bool bSingleEntry)
	{
		FToolUIAction LevelInstanceEditAction;
		FText EntryDesc;
		AActor* LevelInstanceActor = CastChecked<AActor>(LevelInstance);
		const bool bCanEdit = LevelInstance->CanEnterEdit(&EntryDesc);

		LevelInstanceEditAction.ExecuteAction.BindLambda([LevelInstance, ContextActor](const FToolMenuContext&)
		{
			LevelInstance->EnterEdit(ContextActor);
		});
		LevelInstanceEditAction.CanExecuteAction.BindLambda([bCanEdit](const FToolMenuContext&)
		{
			return bCanEdit;
		});

		FText EntryLabel = bSingleEntry ? LOCTEXT("EditLevelInstances", "Edit") : FText::FromString(LevelInstance->GetWorldAsset().GetAssetName());
		if (bCanEdit)
		{
			FText EntryActionDesc = LOCTEXT("EditLevelInstancesPropertyTooltip", "Edit this level. Your changes will be applied to the level asset and to all other level instances based on it.");
			EntryDesc = FText::Format(LOCTEXT("LevelInstanceName", "{0}\n\nActor name: {1}\nAsset path: {2}"), EntryActionDesc, FText::FromString(LevelInstanceActor->GetActorLabel()), FText::FromString(LevelInstance->GetWorldAssetPackage()));
		}
		Section.AddMenuEntry(NAME_None, EntryLabel, EntryDesc, FSlateIcon(), LevelInstanceEditAction);
	}

	static void CreateEditSubMenu(UToolMenu* Menu, TArray<ILevelInstanceInterface*> LevelInstanceHierarchy, AActor* ContextActor)
	{
		FToolMenuSection& Section = Menu->AddSection(NAME_None, LOCTEXT("LevelInstanceContextEditSection", "Context"));
		for (ILevelInstanceInterface* LevelInstance : LevelInstanceHierarchy)
		{
			CreateEditMenuEntry(Section, LevelInstance, ContextActor, false);
		}
	}

	static void CreateEditPropertyOverridesMenuEntry(FToolMenuSection& Section, ILevelInstanceInterface* LevelInstance, AActor* ContextActor, bool bSingleEntry)
	{
		FToolUIAction LevelInstanceEditAction;
		FText EntryDesc;
		AActor* LevelInstanceActor = CastChecked<AActor>(LevelInstance);
		const bool bCanEdit = LevelInstance->CanEnterEditPropertyOverrides(&EntryDesc);

		LevelInstanceEditAction.ExecuteAction.BindLambda([LevelInstance, ContextActor](const FToolMenuContext&)
		{
			LevelInstance->EnterEditPropertyOverrides(ContextActor);
		});
		LevelInstanceEditAction.CanExecuteAction.BindLambda([bCanEdit](const FToolMenuContext&)
		{
			return bCanEdit;
		});

		FText EntryLabel = bSingleEntry ? LOCTEXT("OverrideLevelInstances", "Override") : FText::FromString(LevelInstance->GetWorldAsset().GetAssetName());
		if (bCanEdit)
		{
			// EntryDesc contains the reason the menu cannot be enabled, when CanEnterEditPropertyOverrides would return false. It returned true, so the item will be enabled, and we set another tooltip
			FText EntryActionDesc = LOCTEXT("EditLevelInstancesPropertyOverridesTooltip", "Edit only this level instance, without changing the level asset or any other level instances.");
			EntryDesc = FText::Format(LOCTEXT("OverrideLevelInstanceName", "{0}\n\nActor name: {1}\nAsset path: {2}"), EntryActionDesc, FText::FromString(LevelInstanceActor->GetActorLabel()), FText::FromString(LevelInstance->GetWorldAssetPackage()));
		}
		Section.AddMenuEntry(NAME_None, EntryLabel, EntryDesc, FSlateIcon(), LevelInstanceEditAction);
	}

	static void CreateEditPropertyOverridesSubMenu(UToolMenu* Menu, TArray<ILevelInstanceInterface*> LevelInstanceHierarchy, AActor* ContextActor)
	{
		FToolMenuSection& Section = Menu->AddSection(NAME_None, LOCTEXT("LevelInstanceContextEditSection", "Context"));
		for (ILevelInstanceInterface* LevelInstance : LevelInstanceHierarchy)
		{
			CreateEditPropertyOverridesMenuEntry(Section, LevelInstance, ContextActor, false);
		}
	}
		
	static void MoveSelectionToLevelInstance(ILevelInstanceInterface* DestinationLevelInstance, const TArray<AActor*>& ActorsToMove)
	{
		DestinationLevelInstance->MoveActorsTo(ActorsToMove);
	}
		
	static void CreateEditMenu(UToolMenu* Menu, AActor* ContextActor)
	{
		if (ULevelInstanceSubsystem* LevelInstanceSubsystem = ContextActor->GetWorld()->GetSubsystem<ULevelInstanceSubsystem>())
		{
			TArray<ILevelInstanceInterface*> LevelInstanceHierarchy;
			LevelInstanceSubsystem->ForEachLevelInstanceAncestorsAndSelf(ContextActor, [&LevelInstanceHierarchy](ILevelInstanceInterface* AncestorLevelInstance)
			{
				LevelInstanceHierarchy.Add(AncestorLevelInstance);
				return true;
			});

			// Don't create sub menu if only one Level Instance is available to edit
			if (LevelInstanceHierarchy.Num() == 1)
			{
				FToolMenuSection& Section = CreateLevelSection(Menu);
				CreateEditMenuEntry(Section, LevelInstanceHierarchy[0], ContextActor, true);
			}
			else if(LevelInstanceHierarchy.Num() > 1)
			{
				FToolMenuSection& Section = CreateLevelSection(Menu);
				Section.AddSubMenu(
					"EditLevelInstances",
					LOCTEXT("EditLevelInstances", "Edit"),
					LOCTEXT("EditLevelInstancesPropertyTooltip", "Edit this level. Your changes will be applied to the level asset and to all other level instances based on it."),
					FNewToolMenuDelegate::CreateStatic(&CreateEditSubMenu, MoveTemp(LevelInstanceHierarchy), ContextActor)
				);
			}
		}
	}

	static void CreateEditPropertyOverridesMenu(UToolMenu* Menu, AActor* ContextActor)
	{
		if (ULevelInstanceSubsystem* LevelInstanceSubsystem = ContextActor->GetWorld()->GetSubsystem<ULevelInstanceSubsystem>())
		{
			TArray<ILevelInstanceInterface*> LevelInstanceHierarchy;
			LevelInstanceSubsystem->ForEachLevelInstanceAncestorsAndSelf(ContextActor, [&LevelInstanceHierarchy](ILevelInstanceInterface* AncestorLevelInstance)
			{
				LevelInstanceHierarchy.Add(AncestorLevelInstance);
				return true;
			});

			// Don't create sub menu if only one Level Instance is available to edit
			if (LevelInstanceHierarchy.Num() == 1)
			{
				FToolMenuSection& Section = CreateLevelSection(Menu);
				CreateEditPropertyOverridesMenuEntry(Section, LevelInstanceHierarchy[0], ContextActor, true);
			}
			else if (LevelInstanceHierarchy.Num() > 1)
			{
				FToolMenuSection& Section = CreateLevelSection(Menu);
				Section.AddSubMenu(
					"PropertyOverrideLevelInstances",
					LOCTEXT("EditLevelInstancesPropertyOverrides", "Override"),
					LOCTEXT("EditLevelInstancesPropertyOverridesTooltip", "Edit only this level instance, without changing the level asset or any other level instances."),
					FNewToolMenuDelegate::CreateStatic(&CreateEditPropertyOverridesSubMenu, MoveTemp(LevelInstanceHierarchy), ContextActor)
				);
			}
		}
	}

	static void CreateSaveCancelMenu(UToolMenu* Menu, AActor* ContextActor)
	{
		ILevelInstanceInterface* LevelInstanceEdit = nullptr;
		if (ContextActor)
		{
			if (ULevelInstanceSubsystem* LevelInstanceSubsystem = ContextActor->GetWorld()->GetSubsystem<ULevelInstanceSubsystem>())
			{
				// Commit Property Overrides has priority
				LevelInstanceEdit = LevelInstanceSubsystem->GetEditingPropertyOverridesLevelInstance();
				if (!LevelInstanceEdit)
				{
					LevelInstanceEdit = LevelInstanceSubsystem->GetEditingLevelInstance();
				}
			}
		}

		// Commmit Property Overrides has priority
		if (!LevelInstanceEdit)
		{
			if (ULevelInstanceSubsystem* LevelInstanceSubsystem = GEditor->GetEditorWorldContext().World()->GetSubsystem<ULevelInstanceSubsystem>())
			{
				LevelInstanceEdit = LevelInstanceSubsystem->GetEditingPropertyOverridesLevelInstance();
			}
		}

		// If no Property Overrides found try to find a regular Edit
		if (!LevelInstanceEdit)
		{
			if (ULevelInstanceSubsystem* LevelInstanceSubsystem = GEditor->GetEditorWorldContext().World()->GetSubsystem<ULevelInstanceSubsystem>())
			{
				LevelInstanceEdit = LevelInstanceSubsystem->GetEditingLevelInstance();
			}
		}
						
		if (LevelInstanceEdit)
		{
			FToolMenuSection& Section = CreateCurrentEditSection(Menu);
			if (LevelInstanceEdit->IsEditingPropertyOverrides())
			{
				FText CommitTooltip = LOCTEXT("LevelInstanceCommitPropertyOverridesTooltip", "Stop overriding this level instance and save any changes you've made.");
				const bool bCanCommit = LevelInstanceEdit->CanExitEditPropertyOverrides(/*bDiscardEdits=*/false, &CommitTooltip);

				FToolUIAction CommitAction;
				CommitAction.ExecuteAction.BindLambda([LevelInstanceEdit](const FToolMenuContext&) { LevelInstanceEdit->ExitEditPropertyOverrides(/*bDiscardEdits=*/false); });
				CommitAction.CanExecuteAction.BindLambda([bCanCommit](const FToolMenuContext&) { return bCanCommit; });
				Section.AddMenuEntry(NAME_None, LOCTEXT("LevelInstanceSavePropertyOverridesLabel", "Save Override(s)"), CommitTooltip, FSlateIcon(), CommitAction);

				FText DiscardTooltip = LOCTEXT("LevelInstanceDiscardPropertyOverridesTooltip", "Stop overriding this level instance and discard any changes you've made.");
				const bool bCanDiscard = LevelInstanceEdit->CanExitEditPropertyOverrides(/*bDiscardEdits=*/true, &DiscardTooltip);

				FToolUIAction DiscardAction;
				DiscardAction.ExecuteAction.BindLambda([LevelInstanceEdit](const FToolMenuContext&) { LevelInstanceEdit->ExitEditPropertyOverrides(/*bDiscardEdits=*/true); });
				DiscardAction.CanExecuteAction.BindLambda([bCanDiscard](const FToolMenuContext&) { return bCanDiscard; });
				Section.AddMenuEntry(NAME_None, LOCTEXT("LevelInstanceCancelPropertyOverridesLabel", "Cancel Override(s)"), DiscardTooltip, FSlateIcon(), DiscardAction);
			}
			else
			{
				FText CommitTooltip = LOCTEXT("LevelInstanceCommitTooltip", "Stop editing this level and save any changes you've made.");
				const bool bCanCommit = LevelInstanceEdit->CanExitEdit(/*bDiscardEdits=*/false, &CommitTooltip);

				FToolUIAction CommitAction;
				CommitAction.ExecuteAction.BindLambda([LevelInstanceEdit](const FToolMenuContext&) { LevelInstanceEdit->ExitEdit(/*bDiscardEdits=*/false); });
				CommitAction.CanExecuteAction.BindLambda([bCanCommit](const FToolMenuContext&) { return bCanCommit; });
				Section.AddMenuEntry(NAME_None, LOCTEXT("LevelInstanceSaveLabel", "Save"), CommitTooltip, FSlateIcon(), CommitAction);

				FText DiscardTooltip = LOCTEXT("LevelInstanceDiscardTooltip", "Stop editing this level and discard any changes you've made.");
				const bool bCanDiscard = LevelInstanceEdit->CanExitEdit(/*bDiscardEdits=*/true, &DiscardTooltip);

				FToolUIAction DiscardAction;
				DiscardAction.ExecuteAction.BindLambda([LevelInstanceEdit](const FToolMenuContext&) { LevelInstanceEdit->ExitEdit(/*bDiscardEdits=*/true); });
				DiscardAction.CanExecuteAction.BindLambda([bCanDiscard](const FToolMenuContext&) { return bCanDiscard; });
				Section.AddMenuEntry(NAME_None, LOCTEXT("LevelInstanceCancelLabel", "Cancel"), DiscardTooltip, FSlateIcon(), DiscardAction);
			}
		}
	}

	static UClass* GetDefaultLevelInstanceClass(ELevelInstanceCreationType CreationType)
	{
		if (CreationType == ELevelInstanceCreationType::PackedLevelActor)
		{
			return APackedLevelActor::StaticClass();
		}

		ULevelInstanceEditorSettings* LevelInstanceEditorSettings = GetMutableDefault<ULevelInstanceEditorSettings>();
		if (!LevelInstanceEditorSettings->LevelInstanceClassName.IsEmpty())
		{
			UClass* LevelInstanceClass = LoadClass<AActor>(nullptr, *LevelInstanceEditorSettings->LevelInstanceClassName, nullptr, LOAD_NoWarn);
			if (LevelInstanceClass && LevelInstanceClass->ImplementsInterface(ULevelInstanceInterface::StaticClass()))
			{
				return LevelInstanceClass;
			}
		}

		return ALevelInstance::StaticClass();
	}

	static bool AreAllSelectedLevelInstancesRootSelections(const TArray<ILevelInstanceInterface*>& SelectedLevelInstances)
	{
		for(ILevelInstanceInterface* LevelInstance : SelectedLevelInstances)
		{
			if (CastChecked<AActor>(LevelInstance)->GetSelectionParent() != nullptr)
			{
				return false;
			}
		}

		return true;
	}
		
	static void CreateLevelInstanceFromSelection(ULevelInstanceSubsystem* LevelInstanceSubsystem, ELevelInstanceCreationType CreationType, const TArray<AActor*>& ActorsToMove)
	{
		IMainFrameModule& MainFrameModule = FModuleManager::GetModuleChecked<IMainFrameModule>("MainFrame");

		TSharedPtr<SWindow> NewLevelInstanceWindow =
			SNew(SWindow)
			.Title(FText::Format(LOCTEXT("NewLevelInstanceWindowTitle", "New {0}"), StaticEnum<ELevelInstanceCreationType>()->GetDisplayNameTextByValue((int64)CreationType)))
			.SupportsMinimize(false)
			.SupportsMaximize(false)
			.SizingRule(ESizingRule::Autosized);

		TSharedRef<SNewLevelInstanceDialog> NewLevelInstanceDialog =
			SNew(SNewLevelInstanceDialog)
			.ParentWindow(NewLevelInstanceWindow)
			.PivotActors(ActorsToMove);

		const bool bForceExternalActors = LevelInstanceSubsystem->GetWorld()->IsPartitionedWorld();
		FNewLevelInstanceParams& DialogParams = NewLevelInstanceDialog->GetCreationParams();
		DialogParams.Type = CreationType;
		DialogParams.bAlwaysShowDialog = GetDefault<ULevelInstanceEditorPerProjectUserSettings>()->bAlwaysShowDialog;
		DialogParams.PivotType = GetDefault<ULevelInstanceEditorPerProjectUserSettings>()->PivotType;
		DialogParams.PivotActor = DialogParams.PivotType == ELevelInstancePivotType::Actor ? ActorsToMove[0] : nullptr;
		DialogParams.HideCreationType();
		DialogParams.SetForceExternalActors(bForceExternalActors);
		NewLevelInstanceWindow->SetContent(NewLevelInstanceDialog);

		if (GetDefault<ULevelInstanceEditorPerProjectUserSettings>()->bAlwaysShowDialog)
		{
			FSlateApplication::Get().AddModalWindow(NewLevelInstanceWindow.ToSharedRef(), MainFrameModule.GetParentWindow());
		}
		
		if (!GetDefault<ULevelInstanceEditorPerProjectUserSettings>()->bAlwaysShowDialog || NewLevelInstanceDialog->ClickedOk())
		{
			FNewLevelInstanceParams CreationParams(NewLevelInstanceDialog->GetCreationParams());
			ULevelInstanceEditorPerProjectUserSettings::UpdateFrom(CreationParams);

			FNewLevelDialogModule& NewLevelDialogModule = FModuleManager::LoadModuleChecked<FNewLevelDialogModule>("NewLevelDialog");
			FString TemplateMapPackage;
			bool bOutIsPartitionedWorld = false;
			const bool bShowPartitionedTemplates = false;
			ULevelInstanceEditorSettings* LevelInstanceEditorSettings = GetMutableDefault<ULevelInstanceEditorSettings>();
			if (!LevelInstanceEditorSettings->TemplateMapInfos.Num() || NewLevelDialogModule.CreateAndShowTemplateDialog(MainFrameModule.GetParentWindow(), LOCTEXT("LevelInstanceTemplateDialog", "Choose Level Instance Template..."), GetMutableDefault<ULevelInstanceEditorSettings>()->TemplateMapInfos, TemplateMapPackage, bShowPartitionedTemplates, bOutIsPartitionedWorld))
			{
				UPackage* TemplatePackage = !TemplateMapPackage.IsEmpty() ? LoadPackage(nullptr, *TemplateMapPackage, LOAD_None) : nullptr;
				
				CreationParams.TemplateWorld = TemplatePackage ? UWorld::FindWorldInPackage(TemplatePackage) : nullptr;
				CreationParams.LevelInstanceClass = GetDefaultLevelInstanceClass(CreationType);
				CreationParams.bEnableStreaming = LevelInstanceEditorSettings->bEnableStreaming;

				if (!LevelInstanceSubsystem->CreateLevelInstanceFrom(ActorsToMove, CreationParams))
				{
					FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("CreateFromSelectionFailMsg", "Failed to create from selection. Check log for details."), LOCTEXT("CreateFromSelectionFailTitle", "Create from selection failed"));
				}
			}
		}
	}

	static void CreateCreateMenu(UToolMenu* ToolMenu, const TArray<AActor*>& ActorsToMove)
	{
		if (ULevelInstanceSubsystem* LevelInstanceSubsystem = GEditor->GetEditorWorldContext().World()->GetSubsystem<ULevelInstanceSubsystem>())
		{
			if (LevelInstanceSubsystem->CanCreateLevelInstanceFrom(ActorsToMove))
			{
				FToolMenuSection& Section = ToolMenu->AddSection("ActorSelectionSectionName", LOCTEXT("ActorSelectionSectionLabel", "Actor Selection"));
								
				Section.AddMenuEntry(
					TEXT("CreateLevelInstance"),
					FText::Format(LOCTEXT("CreateFromSelectionLabel", "Create {0}..."), StaticEnum<ELevelInstanceCreationType>()->GetDisplayNameTextByValue((int64)ELevelInstanceCreationType::LevelInstance)),
					TAttribute<FText>(),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.LevelInstance"),
					FExecuteAction::CreateLambda([LevelInstanceSubsystem, CopyActorsToMove = ActorsToMove]
					{
							CreateLevelInstanceFromSelection(LevelInstanceSubsystem, ELevelInstanceCreationType::LevelInstance, CopyActorsToMove);
					}));
				
				Section.AddMenuEntry(
					TEXT("CreatePackedLevelBlueprint"),
					FText::Format(LOCTEXT("CreateFromSelectionLabel", "Create {0}..."), StaticEnum<ELevelInstanceCreationType>()->GetDisplayNameTextByValue((int64)ELevelInstanceCreationType::PackedLevelActor)),
					TAttribute<FText>(),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.PackedLevelActor"),
					FExecuteAction::CreateLambda([LevelInstanceSubsystem, CopyActorsToMove = ActorsToMove]
					{
						CreateLevelInstanceFromSelection(LevelInstanceSubsystem, ELevelInstanceCreationType::PackedLevelActor, CopyActorsToMove);
					}));
				
			}
		}
	}
		
	static void CreateBreakSubMenu(UToolMenu* Menu, const TArray<ILevelInstanceInterface*>& BreakableLevelInstances)
	{
		static int32 BreakLevels = 1;
		ULevelInstanceEditorPerProjectUserSettings* Settings = GetMutableDefault<ULevelInstanceEditorPerProjectUserSettings>();

		if (ULevelInstanceSubsystem* LevelInstanceSubsystem = GEditor->GetEditorWorldContext().World()->GetSubsystem<ULevelInstanceSubsystem>())
		{
			FToolMenuSection& Section = Menu->AddSection("Options", LOCTEXT("LevelInstanceBreakOptionsSection", "Options"));

			FToolMenuEntry OrganizeInFoldersEntry = FToolMenuEntry::InitMenuEntry(
				"OrganizeInFolders",
				LOCTEXT("OrganizeActorsInFolders", "Keep Folders"),
				LOCTEXT(
					"OrganizeActorsInFoldersTooltip",
					"When checked, actors remain in the same folder as the level instance "
					"and use the same folder structure."
					"\nWhen unchecked, actors are placed at the root of the current level's hierarchy."
				),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([Settings]()
					{
						Settings->bKeepFoldersDuringBreak = !Settings->bKeepFoldersDuringBreak;
					}),
					FCanExecuteAction(),
					FGetActionCheckState::CreateLambda([Settings]
					{
						return Settings->bKeepFoldersDuringBreak ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					})
				),
				EUserInterfaceActionType::ToggleButton
			);
			OrganizeInFoldersEntry.bShouldCloseWindowAfterMenuSelection = false;
			Section.AddEntry(OrganizeInFoldersEntry);

			TSharedRef<SWidget> MenuWidget =
				SNew(SBox)
				.Padding(FMargin(5, 2, 5, 0))
				[
					SNew(SNumericEntryBox<int32>)
					.MinValue(1)
					.Value_Lambda([]() { return BreakLevels; })
					.OnValueChanged_Lambda([](int32 InValue) { BreakLevels = InValue; })
					.ToolTipText(LOCTEXT("BreakLevelsTooltip", "Determines the depth of nested instances to break apart. Use 1 to break only the top level instance."))
					.Label()
					[
						SNumericEntryBox<int32>::BuildLabel(LOCTEXT("BreakDepthLabel", "Depth"), FLinearColor::White, FLinearColor::Transparent)
					]
				];

			Section.AddEntry(FToolMenuEntry::InitWidget("SetBreakLevels", MenuWidget, FText::GetEmpty(), false));

			Section.AddSeparator(NAME_None);

			FToolMenuEntry ExecuteEntry = FToolMenuEntry::InitMenuEntry(
				"ExecuteBreak",
				LOCTEXT("BreakLevelInstances_BreakLevelInstanceButton", "Break Level Instance(s)"),
				LOCTEXT("BreakLevelInstances_BreakLevelInstanceButtonTooltip", "Break apart the selected level instances using the settings above."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([CopyBreakableLevelInstances = BreakableLevelInstances, LevelInstanceSubsystem, Settings]()
					{
						const FText LevelInstanceBreakWarning = FText::Format(
							LOCTEXT(
								"BreakingLevelInstance",
								"You are about to break {0} level instance(s). This action cannot be undone. Are you sure ?"
							),
							FText::AsNumber(CopyBreakableLevelInstances.Num())
						);

						if (FMessageDialog::Open(EAppMsgType::YesNo, LevelInstanceBreakWarning, LOCTEXT("BreakingLevelInstanceTitle", "Break Level Instances")) == EAppReturnType::Yes)
						{
							ELevelInstanceBreakFlags Flags = ELevelInstanceBreakFlags::None;
							if (Settings->bKeepFoldersDuringBreak)
							{
								Flags |= ELevelInstanceBreakFlags::KeepFolders;
							}

							for (ILevelInstanceInterface* LevelInstance : CopyBreakableLevelInstances)
							{
								LevelInstanceSubsystem->BreakLevelInstance(LevelInstance, BreakLevels, nullptr, Flags);
							}
						}
					})
				),
				EUserInterfaceActionType::Button
			);

			Section.AddEntry(ExecuteEntry);
		}
	}

	static void CreateBreakMenu(UToolMenu* Menu, const TArray<ILevelInstanceInterface*>& SelectedLevelInstances)
	{
		if(ULevelInstanceSubsystem* LevelInstanceSubsystem = GEditor->GetEditorWorldContext().World()->GetSubsystem<ULevelInstanceSubsystem>())
		{
			TArray<ILevelInstanceInterface*> BreakableLevelInstances;
			for (ILevelInstanceInterface* SelectedLevelInstance : SelectedLevelInstances)
			{
				if (LevelInstanceSubsystem->CanBreakLevelInstance(SelectedLevelInstance))
				{
					BreakableLevelInstances.Add(SelectedLevelInstance);
				}	
			}
			
			if (BreakableLevelInstances.Num())
			{
				FToolMenuSection& Section = CreateLevelSection(Menu);
				Section.AddSubMenu(
					"BreakLevelInstances",
					LOCTEXT("BreakLevelInstances", "Break"),
					LOCTEXT("BreakLevelInstancesTooltip", "Break apart the selected level instances into their individual actors."),
					FNewToolMenuDelegate::CreateLambda([CopyOfBreakableLevelInstances = BreakableLevelInstances](UToolMenu* Menu)
					{
						CreateBreakSubMenu(Menu, CopyOfBreakableLevelInstances);
					}));
			}
		}

	}

	static void CreatePackedBlueprintMenu(UToolMenu* Menu, AActor* ContextActor)
	{
		if (ULevelInstanceSubsystem* LevelInstanceSubsystem = ContextActor->GetWorld()->GetSubsystem<ULevelInstanceSubsystem>())
		{
			ILevelInstanceInterface* ContextLevelInstance = nullptr;

			// Find the top level LevelInstance
			LevelInstanceSubsystem->ForEachLevelInstanceAncestorsAndSelf(ContextActor, [LevelInstanceSubsystem, ContextActor, &ContextLevelInstance](ILevelInstanceInterface* Ancestor)
			{
				if (CastChecked<AActor>(Ancestor)->GetLevel() == ContextActor->GetWorld()->GetCurrentLevel())
				{
					ContextLevelInstance = Ancestor;
					return false;
				}
				return true;
			});
						
			if (ContextLevelInstance && !ContextLevelInstance->IsEditing())
			{
				FToolMenuSection& Section = CreateLevelSection(Menu);
				;
				if (APackedLevelActor* PackedLevelActor = Cast<APackedLevelActor>(ContextLevelInstance))
				{
					if (TSoftObjectPtr<UBlueprint> BlueprintAsset = Cast<UBlueprint>(PackedLevelActor->GetClass()->ClassGeneratedBy.Get()))
					{
						FToolUIAction UIAction;
						UIAction.ExecuteAction.BindLambda([ContextLevelInstance, BlueprintAsset](const FToolMenuContext& MenuContext)
						{
							FPackedLevelActorUtils::CreateOrUpdateBlueprint(ContextLevelInstance->GetWorldAsset(), BlueprintAsset);
						});
						UIAction.CanExecuteAction.BindLambda([](const FToolMenuContext& MenuContext)
						{
							return FPackedLevelActorUtils::CanPack() && GEditor->GetSelectedActorCount() > 0;
						});

						Section.AddMenuEntry(
							"UpdatePackedBlueprint",
							LOCTEXT("UpdatePackedBlueprint", "Update Packed Blueprint"),
							TAttribute<FText>(),
							TAttribute<FSlateIcon>(),
							UIAction);
					}
				}
			}
		}
	}

	static void CreateResetPropertyOverridesMenu(UToolMenu* Menu, const TArray<AActor*>& SelectedActors, const TArray<ILevelInstanceInterface*>& SelectedLevelInstances)
	{
		if (!ULevelInstanceSettings::Get()->IsPropertyOverrideEnabled())
		{
			return;
		}

		if (ULevelInstanceSubsystem* LevelInstanceSubsystem = UWorld::GetSubsystem<ULevelInstanceSubsystem>(GEditor->GetEditorWorldContext().World()))
		{
			if (SelectedLevelInstances.Num() > 0 && SelectedActors.Num() == SelectedLevelInstances.Num())
			{
				bool bCanResetAllLevelInstances = true;
				for (ILevelInstanceInterface* SelectedLevelInstance : SelectedLevelInstances)
				{
					if (!LevelInstanceSubsystem->CanResetPropertyOverrides(SelectedLevelInstance))
					{
						bCanResetAllLevelInstances = false;
						break;
					}
				}

				if (bCanResetAllLevelInstances)
				{
					FToolMenuSection& Section = CreateLevelSection(Menu);
					FToolUIAction UIAction;
					UIAction.ExecuteAction.BindLambda([LevelInstanceSubsystem, CopySelectedLevelInstance = SelectedLevelInstances](const FToolMenuContext& MenuContext)
						{
							for (ILevelInstanceInterface* LevelInstanceInterface : CopySelectedLevelInstance)
							{
								LevelInstanceSubsystem->ResetPropertyOverrides(LevelInstanceInterface);
							}
						});

					Section.AddMenuEntry(
						"ResetLevelInstancePropertyOverrides",
						LOCTEXT("ResetLevelInstancePropertyOverrides", "Reset Overrides"),
						TAttribute<FText>(),
						TAttribute<FSlateIcon>(),
						UIAction);

					return;
				}
			}

			if (SelectedActors.Num() > 0)
			{
				bool bCanResetAllActors = true;
				for (AActor* SelectedActor : SelectedActors)
				{
					if (!LevelInstanceSubsystem->CanResetPropertyOverridesForActor(SelectedActor))
					{
						bCanResetAllActors = false;
						break;
					}
				}

				if (bCanResetAllActors)
				{
					FToolMenuSection& Section = CreateLevelSection(Menu);
					FToolUIAction UIAction;
					UIAction.ExecuteAction.BindLambda([LevelInstanceSubsystem, CopySelectedActors = SelectedActors](const FToolMenuContext& MenuContext)
					{
						FScopedTransaction ResetPropertyOverridesTransaction(LOCTEXT("ResetPropertyOverrides", "Reset Property Override(s)"));
						for (AActor* SelectedActor : CopySelectedActors)
						{
							LevelInstanceSubsystem->ResetPropertyOverridesForActor(SelectedActor);
						}
					});

					Section.AddMenuEntry(
						"ResetLevelInstancePropertyOverrides",
						LOCTEXT("ResetLevelInstancePropertyOverrides", "Reset Overrides"),
						LOCTEXT("ResetLevelInstancePropertyOverridesTooltip", "Discard all overrides on the selected level instances, restoring them to match the level assets."),
						TAttribute<FSlateIcon>(),
						UIAction);
				}
			}
		}
	}

	class FLevelInstanceClassFilter : public IClassViewerFilter
	{
	public:
		
		virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs) override
		{
			return InClass && InClass->ImplementsInterface(ULevelInstanceInterface::StaticClass()) && InClass->IsNative() && !InClass->HasAnyClassFlags(CLASS_Deprecated);
		}

		virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef<const IUnloadedBlueprintData> InUnloadedClassData, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs) override
		{
			return false;
		}
	};

	static void CreateBlueprintFromWorld(UWorld* WorldAsset)
	{
		TSoftObjectPtr<UWorld> LevelInstancePtr(WorldAsset);

		int32 LastSlashIndex = 0;
		FString LongPackageName = LevelInstancePtr.GetLongPackageName();
		LongPackageName.FindLastChar('/', LastSlashIndex);
		
		FString PackagePath = LongPackageName.Mid(0, LastSlashIndex == INDEX_NONE ? MAX_int32 : LastSlashIndex);
		FString AssetName = "BP_" + LevelInstancePtr.GetAssetName();
		IAssetTools& AssetTools = FAssetToolsModule::GetModule().Get();

		UBlueprintFactory* BlueprintFactory = NewObject<UBlueprintFactory>();
		BlueprintFactory->AddToRoot();
		BlueprintFactory->OnConfigurePropertiesDelegate.BindLambda([](FClassViewerInitializationOptions* Options)
		{
			Options->bShowDefaultClasses = false;
			Options->bIsBlueprintBaseOnly = false;
			Options->InitiallySelectedClass = ALevelInstance::StaticClass();
			Options->bIsActorsOnly = true;
			Options->ClassFilters.Add(MakeShareable(new FLevelInstanceClassFilter));
		});
		ON_SCOPE_EXIT
		{
			BlueprintFactory->OnConfigurePropertiesDelegate.Unbind();
			BlueprintFactory->RemoveFromRoot();
		};

		if (UBlueprint* NewBlueprint = Cast<UBlueprint>(AssetTools.CreateAssetWithDialog(AssetName, PackagePath, UBlueprint::StaticClass(), BlueprintFactory, FName("Create LevelInstance Blueprint"))))
		{
			AActor* CDO = NewBlueprint->GeneratedClass->GetDefaultObject<AActor>();
			ILevelInstanceInterface* LevelInstanceCDO = CastChecked<ILevelInstanceInterface>(CDO);
			LevelInstanceCDO->SetWorldAsset(LevelInstancePtr);
			FBlueprintEditorUtils::MarkBlueprintAsModified(NewBlueprint);
			
			if (NewBlueprint->GeneratedClass->IsChildOf<APackedLevelActor>())
			{
				FPackedLevelActorUtils::UpdateBlueprint(NewBlueprint);
			}

			FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
			TArray<UObject*> Assets;
			Assets.Add(NewBlueprint);
			ContentBrowserModule.Get().SyncBrowserToAssets(Assets);
		}		
	}

	static void CreateBlueprintFromMenu(UToolMenu* Menu, FAssetData WorldAsset)
	{
		FToolMenuSection& Section = CreateLevelSection(Menu);
		FToolUIAction UIAction;
		UIAction.ExecuteAction.BindLambda([WorldAsset](const FToolMenuContext& MenuContext)
		{
			if (UWorld* World = Cast<UWorld>(WorldAsset.GetAsset()))
			{
				CreateBlueprintFromWorld(World);
			}
		});

		Section.AddMenuEntry(
			"CreateLevelInstanceBlueprint",
			LOCTEXT("CreateLevelInstanceBlueprint", "New Blueprint..."),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.LevelInstance"),
			UIAction);
	}

	static void AddPartitionedStreamingSupportFromWorld(UWorld* WorldAsset)
	{
		if (WorldAsset->GetStreamingLevels().Num())
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("AddPartitionedLevelInstanceStreamingSupportError_SubLevels", "Cannot convert this world has it contains sublevels."));
			return;
		}

		if (WorldAsset->WorldType != EWorldType::Inactive)
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("AddPartitionedLevelInstanceStreamingSupportError_Loaded", "Cannot convert this world has it's already loaded in the editor."));
			return;
		}

		bool bSuccess = false;
		UWorld* World = GEditor->GetEditorWorldContext().World();
		ULevelInstanceSubsystem::ResetLoadersForWorldAsset(WorldAsset->GetPackage()->GetName());

		FWorldPartitionConverter::FParameters Parameters;
		Parameters.bConvertSubLevels = false;
		Parameters.bEnableStreaming = false;
		Parameters.bUseActorFolders = true;

		if (FWorldPartitionConverter::Convert(WorldAsset, Parameters))
		{
			TArray<UPackage*> PackagesToSave = WorldAsset->PersistentLevel->GetLoadedExternalObjectPackages();
			TSet<UPackage*> PackagesToSaveSet(PackagesToSave);

			PackagesToSaveSet.Add(WorldAsset->GetPackage());

			const bool bPromptUserToSave = false;
			const bool bSaveMapPackages = true;
			const bool bSaveContentPackages = true;
			const bool bFastSave = false;
			const bool bNotifyNoPackagesSaved = false;
			const bool bCanBeDeclined = true;

			if (FEditorFileUtils::SaveDirtyPackages(bPromptUserToSave, bSaveMapPackages, bSaveContentPackages, bFastSave, bNotifyNoPackagesSaved, bCanBeDeclined, nullptr, [&PackagesToSaveSet](UPackage* PackageToSave) { return !PackagesToSaveSet.Contains(PackageToSave); }))
			{
				bSuccess = true;
				for (UPackage* PackageToSave : PackagesToSave)
				{
					if (PackageToSave->IsDirty())
					{
						UE_LOGF(LogLevelInstanceEditor, Error, "Package '%ls' failed to save", *PackageToSave->GetName());
						bSuccess = false;
						break;
					}
				}
			}
		}

		if (!bSuccess)
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("AddPartitionedLevelInstanceStreamingSupportError", "An error occured when adding partitioned level instance streaming support, check logs for details.."));
		}
	}

	static void UpdatePackedBlueprintsFromMenu(UToolMenu* Menu, FAssetData WorldAsset)
	{
		FToolMenuSection& Section = CreateLevelSection(Menu);
		FToolUIAction UIAction;
		UIAction.CanExecuteAction.BindLambda([](const FToolMenuContext& MenuContext)
		{
			return FPackedLevelActorUtils::CanPack();
		});
		UIAction.ExecuteAction.BindLambda([WorldAsset](const FToolMenuContext& MenuContext)
		{
			return FPackedLevelActorUtils::UpdateAllPackedBlueprintsForWorldAssetBlueprint(TSoftObjectPtr<UWorld>(WorldAsset.GetSoftObjectPath()));
		});

		Section.AddMenuEntry(
			"UpdatePackedBlueprintsFromMenu",
			LOCTEXT("UpdatePackedBlueprintsFromMenu", "Update Packed Blueprints"),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.PackedLevelActor"),
			UIAction
		);
	}

	static void AddPartitionedStreamingSupportFromMenu(UToolMenu* Menu, FAssetData WorldAsset)
	{
		FName WorldAssetName = WorldAsset.PackageName;
		if (!ULevel::GetIsLevelPartitionedFromPackage(WorldAssetName))
		{
			FToolMenuSection& Section = CreateLevelSection(Menu);
			FToolUIAction UIAction;
			UIAction.ExecuteAction.BindLambda([WorldAsset](const FToolMenuContext& MenuContext)
			{
				if (UWorld* World = Cast<UWorld>(WorldAsset.GetAsset()))
				{
					AddPartitionedStreamingSupportFromWorld(World);
				}
			});

			Section.AddMenuEntry(
				"AddPartitionedStreamingSupportFromMenu",
				LOCTEXT("AddPartitionedStreamingSupportFromMenu", "Add Partitioned Streaming Support"),
				TAttribute<FText>(),
				TAttribute<FSlateIcon>(),
				UIAction
			);
		}
	}
};

class FLevelInstanceActorDetailsSCSEditorUICustomization : public ISCSEditorUICustomization
{
public:
	static TSharedPtr<FLevelInstanceActorDetailsSCSEditorUICustomization> GetInstance()
	{
		if (!Instance)
		{
			Instance = MakeShareable(new FLevelInstanceActorDetailsSCSEditorUICustomization());
		}
		return Instance;
	}

	virtual bool HideComponentsTree(TArrayView<UObject*> Context) const override { return false; }

	virtual bool HideComponentsFilterBox(TArrayView<UObject*> Context) const override { return false; }

	virtual bool HideAddComponentButton(TArrayView<UObject*> Context) const override { return ShouldHide(Context); }

	virtual bool HideBlueprintButtons(TArrayView<UObject*> Context) const override { return ShouldHide(Context); }
private:
	bool ShouldHide(TArrayView<UObject*> Context) const
	{
		for (const UObject* ContextObject : Context)
		{
			if (const AActor* ActorContext = Cast<AActor>(ContextObject))
			{
				if (ActorContext->IsInLevelInstance() && !ActorContext->IsInEditLevelInstance())
				{
					return true;
				}
			}
		}

		return false;
	}
		
	static TSharedPtr<FLevelInstanceActorDetailsSCSEditorUICustomization> Instance;
	bool bShouldHide = false;
};

TSharedPtr<FLevelInstanceActorDetailsSCSEditorUICustomization> FLevelInstanceActorDetailsSCSEditorUICustomization::Instance;

void FLevelInstanceEditorModule::OnLevelEditorCreated(TSharedPtr<ILevelEditor> InLevelEditor)
{
	RegisterToFirstLevelEditor();
}

void FLevelInstanceEditorModule::OnCreateActorBrowserColumns(FSceneOutlinerInitializationOptions& InitOptions, UWorld* World)
{
	if (GetDefault<ULevelInstanceSettings>()->IsPropertyOverrideEnabled())
	{
		FSceneOutlinerColumnInfo ColumnInfo(ESceneOutlinerColumnVisibility::Invisible, 8,
			FCreateSceneOutlinerColumn::CreateRaw(this, &FLevelInstanceEditorModule::CreateLevelInstanceColumn),
			true, TOptional<float>(), LOCTEXT("LevelInstanceColumnName", "Level Instance Overrides"));

		InitOptions.ColumnMap.Add(FLevelInstanceSceneOutlinerColumn::GetID(), ColumnInfo);
	}
}

void FLevelInstanceEditorModule::RegisterToFirstLevelEditor()
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	TSharedPtr<ILevelEditor> FirstLevelEditor = LevelEditorModule.GetFirstLevelEditor();
	if (FirstLevelEditor.IsValid())
	{
		FirstLevelEditor->AddActorDetailsSCSEditorUICustomization(FLevelInstanceActorDetailsSCSEditorUICustomization::GetInstance());

		FEditorModeTools& LevelEditorModeManager = FirstLevelEditor->GetEditorModeManager();
		LevelEditorModeManager.OnEditorModeIDChanged().AddRaw(this, &FLevelInstanceEditorModule::OnEditorModeIDChanged);

		// Create a Behavior source for the default EdModeTools (when we aren't in the LevelInstanceEditorMode)
		DefaultBehaviorSource = ULevelInstanceEditorMode::CreateDefaultModeBehaviorSource(LevelEditorModeManager.GetInteractiveToolsContext());
		LevelEditorModeManager.GetInteractiveToolsContext()->InputRouter->RegisterSource(DefaultBehaviorSource.GetInterface());
		
		// Make sure to unregister because changing the layout will callback on this again.
		// 
		// This works because we aren't actually hooking ourselves to the ILevelEditor but on managers that are shared by the different instances
		// of ILevelEditor. Ideally we could listen to an event when a ILevelEditor gets destroyed to unregister ourselves and continue to listen
		// to this event to re-register ourselves
		LevelEditorModule.OnLevelEditorCreated().RemoveAll(this);
	}
}

FTopLevelAssetPath FLevelInstanceEditorModule::GetBrowseToLevelInstanceAsset(const UObject* Object) const
{
	// Level instances browse to both the level instance asset and the current level asset by default, while we only want to browse to the former
	const ILevelInstanceInterface* LevelInstanceInterface = CastChecked<ILevelInstanceInterface>(Object);
	return LevelInstanceInterface->GetWorldAsset().GetUniqueID().GetAssetPath();
}

void FLevelInstanceEditorModule::StartupModule()
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	if (TSharedPtr<ILevelEditor> FirstLevelEditor = LevelEditorModule.GetFirstLevelEditor())
	{
		RegisterToFirstLevelEditor();
	}
	else
	{
		LevelEditorModule.OnLevelEditorCreated().AddRaw(this, &FLevelInstanceEditorModule::OnLevelEditorCreated);
	}

	FSceneOutlinerModule& SceneOutlinerModule = FModuleManager::LoadModuleChecked<FSceneOutlinerModule>("SceneOutliner");
	SceneOutlinerModule.OnCreateActorBrowserColumns().AddRaw(this, &FLevelInstanceEditorModule::OnCreateActorBrowserColumns);
			
	ExtendContextMenu();

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomClassLayout("LevelInstance", FOnGetDetailCustomizationInstance::CreateStatic(&FLevelInstanceActorDetails::MakeInstance));
	PropertyModule.RegisterCustomClassLayout("LevelInstancePivot", FOnGetDetailCustomizationInstance::CreateStatic(&FLevelInstancePivotDetails::MakeInstance));		
	PropertyModule.RegisterCustomPropertyTypeLayout("WorldPartitionActorFilter", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FLevelInstanceFilterPropertyTypeCustomization::MakeInstance, false), MakeShared<FLevelInstancePropertyTypeIdentifier>(false));
	PropertyModule.RegisterCustomPropertyTypeLayout("WorldPartitionActorFilter", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FLevelInstanceFilterPropertyTypeCustomization::MakeInstance, true), MakeShared<FLevelInstancePropertyTypeIdentifier>(true));
	PropertyModule.NotifyCustomizationModuleChanged();

	// GEditor needs to be set before this module is loaded
	check(GEditor);
	GEditor->OnLevelActorDeleted().AddRaw(this, &FLevelInstanceEditorModule::OnLevelActorDeleted);
	
	EditorLevelUtils::CanMoveActorToLevelDelegate.AddRaw(this, &FLevelInstanceEditorModule::CanMoveActorToLevel);

	// Register actor descriptor loading filter
	class FLevelInstanceActorDescFilter : public IWorldPartitionActorLoaderInterface::FActorDescFilter
	{
	public:
		bool PassFilter(class UWorld* InWorld, const FWorldPartitionHandle& InHandle) override
		{
			if (UWorld* OwningWorld = InWorld->PersistentLevel->GetWorld())
			{
				if (ULevelInstanceSubsystem* LevelInstanceSubsystem = OwningWorld->GetSubsystem<ULevelInstanceSubsystem>())
				{
					return LevelInstanceSubsystem->PassLevelInstanceFilter(InWorld, InHandle);
				}
			}

			return true;
		}

		// Leave [0, 19] for Game code
		virtual uint32 GetFilterPriority() const override { return 20; }

		virtual FText* GetFilterReason() const override
		{
			static FText UnloadedReason(LOCTEXT("LevelInstanceActorDescFilterReason", "Filtered"));
			return &UnloadedReason;
		}
	};
	IWorldPartitionActorLoaderInterface::RegisterActorDescFilter(MakeShareable<IWorldPartitionActorLoaderInterface::FActorDescFilter>(new FLevelInstanceActorDescFilter()));

	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	FMessageLogInitializationOptions InitOptions;
	InitOptions.bShowFilters = true;
	InitOptions.bShowPages = false;
	InitOptions.bAllowClear = true;
	MessageLogModule.RegisterLogListing("PackedLevelActor", LOCTEXT("PackedLevelActorLog", "Packed Level Actor Log"), InitOptions);
		
	FLevelInstanceEditorModeCommands::Register();

	ULevelInstanceSubsystem::RegisterPrimitiveColorHandler();

	if (UBrowseToAssetOverrideSubsystem* BrowseToAssetOverrideSubsystem = UBrowseToAssetOverrideSubsystem::Get())
	{
		BrowseToAssetOverrideSubsystem->RegisterBrowseToAssetOverrideForInterface<ILevelInstanceInterface>(
			FGetBrowseToAssetOverrideDelegate::CreateRaw(this, &FLevelInstanceEditorModule::GetBrowseToLevelInstanceAsset));
	}
}

void FLevelInstanceEditorModule::ShutdownModule()
{
	if (UBrowseToAssetOverrideSubsystem* BrowseToAssetOverrideSubsystem = UBrowseToAssetOverrideSubsystem::Get())
	{
		BrowseToAssetOverrideSubsystem->UnregisterBrowseToAssetOverrideForInterface<ILevelInstanceInterface>();
	}
	
	ULevelInstanceSubsystem::UnregisterPrimitiveColorHandler();

	if (FModuleManager::Get().IsModuleLoaded("LevelEditor"))
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
		LevelEditorModule.OnLevelEditorCreated().RemoveAll(this);
		if (TSharedPtr<ILevelEditor> FirstLevelEditor = LevelEditorModule.GetFirstLevelEditor())
		{
			FirstLevelEditor->RemoveActorDetailsSCSEditorUICustomization(FLevelInstanceActorDetailsSCSEditorUICustomization::GetInstance());
			if (!IsEngineExitRequested())
			{
				FirstLevelEditor->GetEditorModeManager().OnEditorModeIDChanged().RemoveAll(this);
				FirstLevelEditor->GetEditorModeManager().GetInteractiveToolsContext()->InputRouter->DeregisterSource(DefaultBehaviorSource.GetInterface());
			}
		}

		DefaultBehaviorSource = nullptr;
	}

	if (FSceneOutlinerModule* SceneOutlinerModule = FModuleManager::GetModulePtr<FSceneOutlinerModule>("SceneOutliner"))
	{
		SceneOutlinerModule->OnCreateActorBrowserColumns().RemoveAll(this);
	}

	if (GEditor)
	{
		GEditor->OnLevelActorDeleted().RemoveAll(this);
	}

	EditorLevelUtils::CanMoveActorToLevelDelegate.RemoveAll(this);
}

TSharedRef<ISceneOutlinerColumn> FLevelInstanceEditorModule::CreateLevelInstanceColumn(ISceneOutliner& SceneOutliner) const
{
	return MakeShareable(new FLevelInstanceSceneOutlinerColumn(SceneOutliner));
}

void FLevelInstanceEditorModule::OnEditorModeIDChanged(const FEditorModeID& InModeID, bool bIsEnteringMode)
{
	if (InModeID == ULevelInstanceEditorMode::EM_LevelInstanceEditorModeId && !bIsEnteringMode)
	{
		ExitEditorModeEvent.Broadcast();
	}
}

void FLevelInstanceEditorModule::BroadcastTryExitEditorMode() 
{
	TryExitEditorModeEvent.Broadcast();
}

void FLevelInstanceEditorModule::UpdateEditorMode(bool bActivated)
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	if (TSharedPtr<ILevelEditor> FirstLevelEditor = LevelEditorModule.GetFirstLevelEditor())
	{
		if (bActivated && !FirstLevelEditor->GetEditorModeManager().IsModeActive(ULevelInstanceEditorMode::EM_LevelInstanceEditorModeId))
		{
			FirstLevelEditor->GetEditorModeManager().ActivateMode(ULevelInstanceEditorMode::EM_LevelInstanceEditorModeId);
		}
		else if (!bActivated && FirstLevelEditor->GetEditorModeManager().IsModeActive(ULevelInstanceEditorMode::EM_LevelInstanceEditorModeId))
		{
			FirstLevelEditor->GetEditorModeManager().DeactivateMode(ULevelInstanceEditorMode::EM_LevelInstanceEditorModeId);
		}
	}
}

void FLevelInstanceEditorModule::OnLevelActorDeleted(AActor* Actor)
{
	if (ULevelInstanceSubsystem* LevelInstanceSubsystem = Actor->GetWorld()->GetSubsystem<ULevelInstanceSubsystem>())
	{
		LevelInstanceSubsystem->OnActorDeleted(Actor);
	}
}

void FLevelInstanceEditorModule::CanMoveActorToLevel(const AActor* ActorToMove, const ULevel* DestLevel, bool& bOutCanMove)
{
	if (UWorld* World = ActorToMove->GetWorld())
	{
		if (ULevelInstanceSubsystem* LevelInstanceSubsystem = World->GetSubsystem<ULevelInstanceSubsystem>())
		{
			if (!LevelInstanceSubsystem->CanMoveActorToLevel(ActorToMove))
			{
				bOutCanMove = false;
				return;
			}
		}
	}
}

void FLevelInstanceEditorModule::ExtendContextMenu()
{
	if (UToolMenu* BuildMenu = UToolMenus::Get()->ExtendMenu("LevelEditor.MainMenu.Build"))
	{
		FToolMenuSection& Section = BuildMenu->AddSection("LevelEditorLevelInstance", LOCTEXT("PackedLevelActorsHeading", "Packed Level Actor"));
		FUIAction PackAction(
			FExecuteAction::CreateLambda([]() 
			{
				FPackedLevelActorUtils::PackAllLoadedActors();
			}), 
			FCanExecuteAction::CreateLambda([]()
			{
				return FPackedLevelActorUtils::CanPack();
			}),
			FIsActionChecked(),
			FIsActionButtonVisible());

		FToolMenuEntry& Entry = Section.AddMenuEntry(NAME_None, LOCTEXT("PackLevelActorsTitle", "Pack Level Actors"),
			LOCTEXT("PackLevelActorsTooltip", "Update packed level actor blueprints"), FSlateIcon(), PackAction, EUserInterfaceActionType::Button);
	}

	auto AddDynamicSection = [](UToolMenu* ToolMenu)
	{				
		if (GEditor->GetPIEWorldContext())
		{
			return;
		}

		if (GetDefault<ULevelInstanceSettings>()->IsLevelInstanceDisabled())
		{
			return;
		}

		// Build Selection for Menus
		TArray<AActor*> SelectedActors;
		TArray<ILevelInstanceInterface*> SelectedLevelInstances;
		SelectedActors.Reserve(GEditor->GetSelectedActorCount());
		SelectedLevelInstances.Reserve(GEditor->GetSelectedActorCount());
		for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
		{
			if (AActor* Actor = Cast<AActor>(*It); IsValid(Actor))
			{
				SelectedActors.Add(Actor);

				if (ILevelInstanceInterface* LevelInstance = Cast<ILevelInstanceInterface>(Actor))
				{
					SelectedLevelInstances.Add(LevelInstance);
				}
			}
		}

		// Some actions aren't allowed on non root selection Level Instances (Readonly Level Instances)
		const bool bAreAllSelectedLevelInstancesRootSelections = FLevelInstanceMenuUtils::AreAllSelectedLevelInstancesRootSelections(SelectedLevelInstances);

		if (ULevelEditorContextMenuContext* LevelEditorMenuContext = ToolMenu->Context.FindContext<ULevelEditorContextMenuContext>())
		{
			// Use the actor under the cursor if available (e.g. right-click menu).
			// Otherwise use the first selected actor if there's one (e.g. Actor pulldown menu or outliner).
			AActor* ContextActor = LevelEditorMenuContext->HitProxyActor.Get();
			if (!ContextActor && SelectedActors.Num() > 0)
			{
				ContextActor = SelectedActors[0];
			}

			if (ContextActor)
			{
				// Allow Edit/Commmit on non root selected Level Instance
				FLevelInstanceMenuUtils::CreateEditMenu(ToolMenu, ContextActor);
				FLevelInstanceMenuUtils::CreateEditPropertyOverridesMenu(ToolMenu, ContextActor);
				FLevelInstanceMenuUtils::CreateSaveCancelMenu(ToolMenu, ContextActor);
				
				if (bAreAllSelectedLevelInstancesRootSelections)
				{
					FLevelInstanceMenuUtils::CreatePackedBlueprintMenu(ToolMenu, ContextActor);
				}
			}
		}

		if (bAreAllSelectedLevelInstancesRootSelections)
		{
			FLevelInstanceMenuUtils::CreateBreakMenu(ToolMenu, SelectedLevelInstances);
			FLevelInstanceMenuUtils::CreateCreateMenu(ToolMenu, SelectedActors);
			FLevelInstanceMenuUtils::CreateResetPropertyOverridesMenu(ToolMenu, SelectedActors, SelectedLevelInstances);
		}
	};

	if (UToolMenu* ToolMenu = UToolMenus::Get()->ExtendMenu("LevelEditor.ActorContextMenu.LevelSubMenu"))
	{
		ToolMenu->AddDynamicSection("LevelInstanceEditorModuleDynamicSection", FNewToolMenuDelegate::CreateLambda(AddDynamicSection));
	}

	if (UToolMenu* ToolMenu = UToolMenus::Get()->ExtendMenu("LevelEditor.LevelEditorSceneOutliner.ContextMenu.LevelSubMenu"))
	{
		ToolMenu->AddDynamicSection("LevelInstanceEditorModuleDynamicSection", FNewToolMenuDelegate::CreateLambda(AddDynamicSection));
	}

	if (UToolMenu* ToolMenu = UToolMenus::Get()->ExtendMenu("LevelEditor.TedsOutliner.ContextMenu.LevelSubMenu"))
	{
		ToolMenu->AddDynamicSection("LevelInstanceEditorModuleDynamicSection", FNewToolMenuDelegate::CreateLambda(AddDynamicSection));
	}

	if (UToolMenu* WorldAssetMenu = UToolMenus::Get()->ExtendMenu("ContentBrowser.AssetContextMenu.World"))
	{
		FToolMenuSection& Section = WorldAssetMenu->AddDynamicSection("ActorLevelInstance", FNewToolMenuDelegate::CreateLambda([this](UToolMenu* ToolMenu)
		{
			if (GEditor->GetPIEWorldContext())
			{
				return;
			}

			if (GetDefault<ULevelInstanceSettings>()->IsLevelInstanceDisabled())
			{
				return;
			}

			if (ToolMenu)
			{
				if (UContentBrowserAssetContextMenuContext* AssetMenuContext = ToolMenu->Context.FindContext<UContentBrowserAssetContextMenuContext>())
				{
					if (AssetMenuContext->SelectedAssets.Num() != 1)
					{
						return;
					}

					const FAssetData& WorldAsset = AssetMenuContext->SelectedAssets[0];
					if (AssetMenuContext->SelectedAssets[0].IsInstanceOf<UWorld>())
					{
						FLevelInstanceMenuUtils::CreateBlueprintFromMenu(ToolMenu, WorldAsset);
						FLevelInstanceMenuUtils::UpdatePackedBlueprintsFromMenu(ToolMenu, WorldAsset);
						FLevelInstanceMenuUtils::AddPartitionedStreamingSupportFromMenu(ToolMenu, WorldAsset);
					}
				}
			}
		}), FToolMenuInsert(NAME_None, EToolMenuInsertType::Default));
	}
}

bool FLevelInstanceEditorModule::IsEditInPlaceStreamingEnabled() const
{
	return GetDefault<ULevelInstanceEditorSettings>()->bIsEditInPlaceStreamingEnabled;
}

bool FLevelInstanceEditorModule::IsSubSelectionEnabled() const
{
	return GetDefault<ULevelInstanceEditorPerProjectUserSettings>()->bIsSubSelectionEnabled;
}

void FLevelInstanceEditorModule::AddReferencedObjects(FReferenceCollector& Collector)
{
	DefaultBehaviorSource.AddReferencedObjects(Collector);
}

void FLevelInstanceEditorModule::UpdateAllPackedLevelActorsForWorldAsset(const TSoftObjectPtr<UWorld>& InWorldAsset, bool bInLoadedOnly)
{
	if (FPackedLevelActorUtils::CanPack())
	{
		FPackedLevelActorUtils::UpdateAllPackedBlueprintsForWorldAssetBlueprint(InWorldAsset, bInLoadedOnly);
	}
}
#undef LOCTEXT_NAMESPACE
