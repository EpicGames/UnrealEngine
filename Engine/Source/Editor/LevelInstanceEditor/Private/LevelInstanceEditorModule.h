// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "LevelInstance/ILevelInstanceEditorModule.h"
#include "UObject/GCObject.h"
#include "PropertyEditorArchetypePolicy.h"
#include "PropertyEditorEditConstPolicy.h"
#include "Tools/Modes.h"

class AActor;
class ULevel;
class IInputBehaviorSource;
enum class EMapChangeType : uint8;
class ILevelEditor;
class ISceneOutliner;
class ISceneOutlinerColumn;
struct FSceneOutlinerInitializationOptions;

/**
 * The module holding all of the UI related pieces for LevelInstance management
 */
class FLevelInstanceEditorModule : public ILevelInstanceEditorModule, public FGCObject
{
public:
	virtual ~FLevelInstanceEditorModule()= default;

	/**
	 * Called right after the module DLL has been loaded and the module object has been created
	 */
	virtual void StartupModule();

	/**
	 * Called before the module is unloaded, right before the module object is destroyed.
	 */
	virtual void ShutdownModule();

	virtual void BroadcastTryExitEditorMode() override;

	DECLARE_DERIVED_EVENT(FLevelInstanceEditorModule, ILevelInstanceEditorModule::FExitEditorModeEvent, FExitEditorModeEvent);
	virtual FExitEditorModeEvent& OnExitEditorMode() override { return ExitEditorModeEvent; }

	DECLARE_DERIVED_EVENT(FLevelInstanceEditorModule, ILevelInstanceEditorModule::FTryExitEditorModeEvent, FTryExitEditorModeEvent);
	virtual FTryExitEditorModeEvent& OnTryExitEditorMode() override { return TryExitEditorModeEvent; }

	virtual FOnLevelInstanceEditModeChanged& OnLevelInstanceEditModeChanged() override { return OnLevelInstanceEditModeChangedDelegate; }
	virtual void BroadcastOnLevelInstanceEditModeChanged(ILevelInstanceInterface* LevelInstance, ELevelInstanceEditMode NewMode) override { OnLevelInstanceEditModeChangedDelegate.Broadcast(LevelInstance, NewMode); }

	virtual bool IsEditInPlaceStreamingEnabled() const override;
	virtual bool IsSubSelectionEnabled() const override;

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override { return TEXT("LevelInstanceEditorModule"); }

	virtual void UpdateAllPackedLevelActorsForWorldAsset(const TSoftObjectPtr<UWorld>& InWorldAsset, bool bInLoadedOnly = false) override;

private:
	virtual void UpdateEditorMode(bool bActivated) override;
	
	void OnEditorModeIDChanged(const FEditorModeID& InModeID, bool bIsEnteringMode);
	void OnLevelActorDeleted(AActor* Actor);
	void CanMoveActorToLevel(const AActor* ActorToMove, const ULevel* DestLevel, bool& bOutCanMove);

	void ExtendContextMenu();
		
	void OnLevelEditorCreated(TSharedPtr<ILevelEditor> InLevelEditor);
	void OnCreateActorBrowserColumns(FSceneOutlinerInitializationOptions& InitOptions, UWorld* World);
	void RegisterToFirstLevelEditor();
	FTopLevelAssetPath GetBrowseToLevelInstanceAsset(const UObject* Object) const;

	TSharedRef<ISceneOutlinerColumn> CreateLevelInstanceColumn(ISceneOutliner& SceneOutliner) const;

	class FPropertyEditorPolicy : public PropertyEditorPolicy::IEditConstPolicy, 
								  public PropertyEditorPolicy::IArchetypePolicy
	{
	public:
		FPropertyEditorPolicy(ILevelInstanceEditorModule::IPropertyOverridePolicy* InPropertyOverridePolicy)
			: PropertyOverridePolicy(InPropertyOverridePolicy)
		{
			check(PropertyOverridePolicy);
			PropertyEditorPolicy::RegisterEditConstPolicy(this);
			PropertyEditorPolicy::RegisterArchetypePolicy(this);
		}

		virtual ~FPropertyEditorPolicy()
		{
			PropertyEditorPolicy::UnregisterEditConstPolicy(this);
			PropertyEditorPolicy::UnregisterArchetypePolicy(this);
		}

		virtual UObject* GetArchetypeForObject(const UObject* Object) const override
		{
			return PropertyOverridePolicy->GetArchetypeForObject(Object);
		}

		virtual bool CanEditProperty(const FEditPropertyChain& PropertyChain, const UObject* Object) const override
		{
			return PropertyOverridePolicy->CanEditProperty(PropertyChain, Object);
		}

		virtual bool CanEditProperty(const FProperty* Property, const UObject* Object) const override
		{
			return PropertyOverridePolicy->CanEditProperty(Property, Object);
		}

		ILevelInstanceEditorModule::IPropertyOverridePolicy* PropertyOverridePolicy = nullptr;
	};

	TUniquePtr<FPropertyEditorPolicy> PropertyEditorPolicy;
		
	virtual bool IsPropertyEditConst(const FEditPropertyChain& PropertyChain, UObject* Object) override
	{
		return PropertyEditorPolicy::IsPropertyEditConst(PropertyChain, Object);
	}
	
	virtual bool IsPropertyEditConst(const FProperty* Property, UObject* Object) override
	{
		return PropertyEditorPolicy::IsPropertyEditConst(Property, Object);
	}

	virtual UObject* GetArchetype(const UObject* Object) override
	{
		return PropertyEditorPolicy::GetArchetype(Object);
	}
	
	virtual void SetPropertyOverridePolicy(ILevelInstanceEditorModule::IPropertyOverridePolicy* InPropertyOverridePolicy) override
	{
		PropertyEditorPolicy.Reset();
		if (InPropertyOverridePolicy)
		{
			PropertyEditorPolicy = MakeUnique<FPropertyEditorPolicy>(InPropertyOverridePolicy);
		}
	}
	
	FExitEditorModeEvent ExitEditorModeEvent;
	FTryExitEditorModeEvent TryExitEditorModeEvent;

	FOnLevelInstanceEditModeChanged OnLevelInstanceEditModeChangedDelegate;

	TScriptInterface<IInputBehaviorSource> DefaultBehaviorSource;
};
