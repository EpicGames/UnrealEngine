// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelInstanceEditorSettings.h"
#include "LevelInstance/LevelInstanceActor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelInstanceEditorSettings)

ULevelInstanceEditorSettings::ULevelInstanceEditorSettings()
{
	LevelInstanceClassName = ALevelInstance::StaticClass()->GetPathName();
	bEnableStreaming = false;
	bIsEditInPlaceStreamingEnabled = false;
}

ULevelInstanceEditorPerProjectUserSettings::ULevelInstanceEditorPerProjectUserSettings()
{
	bAlwaysShowDialog = true;
	PivotType = ELevelInstancePivotType::CenterMinZ;
}

void ULevelInstanceEditorPerProjectUserSettings::UpdateFrom(const FNewLevelInstanceParams& Params)
{
	ULevelInstanceEditorPerProjectUserSettings* UserSettings = GetMutableDefault<ULevelInstanceEditorPerProjectUserSettings>();

	if (Params.PivotType != UserSettings->PivotType || Params.bAlwaysShowDialog != UserSettings->bAlwaysShowDialog)
	{
		UserSettings->bAlwaysShowDialog = Params.bAlwaysShowDialog;
		UserSettings->PivotType = Params.PivotType;
		UserSettings->SaveConfig();
	}
}
