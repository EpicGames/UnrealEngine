// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelInstanceFilterPropertyTypeCustomization.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "LevelInstance/LevelInstanceComponent.h"
#include "WorldPartition/WorldPartitionSubsystem.h"
#include "WorldPartition/Filter/WorldPartitionActorFilter.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "ScopedTransaction.h"
#include "Editor.h"
#include "BlueprintEditor.h"
#include "PreviewScene.h"
#include "PropertyHandle.h"
#include "Subsystems/AssetEditorSubsystem.h"

#define LOCTEXT_NAMESPACE "WorldPartitionActorFilter"

TSharedPtr<FWorldPartitionActorFilterMode::FFilter> FLevelInstanceFilterPropertyTypeCustomization::CreateModeFilter(TArray<UObject*> OuterObjects)
{
	LevelInstances.Empty();
	FString ActorLabel;

	// Find Selected Level Instances with matching WorldAssetPackage
	FString WorldAssetPackage;
	UWorld* World = nullptr;
	TOptional<EWorldPartitionActorFilterType> Types;

	for (int32 OuterObjectIndex = 0; OuterObjectIndex < OuterObjects.Num(); ++OuterObjectIndex)
	{
		UObject* OuterObject = OuterObjects[OuterObjectIndex];
		if (AActor* OuterActor = OuterObject->GetTypedOuter<AActor>())
		{
			if (ILevelInstanceInterface* LevelInstanceInterface = Cast<ILevelInstanceInterface>(OuterActor))
			{
				// Make sure this Customization supports current filter type
				if (LevelInstanceInterface->IsEditing() != bIsEditFilter)
				{
					return nullptr;
				}

				LevelInstances.Add(LevelInstanceInterface, OuterObjectIndex);
				ActorLabel = OuterActor->GetActorLabel();
				
				if (!Types.IsSet())
				{
					Types = LevelInstanceInterface->GetDetailsFilterTypes();
					if (Types == EWorldPartitionActorFilterType::None)
					{
						return nullptr;
					}
				}
				else if (Types != LevelInstanceInterface->GetDetailsFilterTypes())
				{
					return nullptr;
				}
				
				if (WorldAssetPackage.IsEmpty())
				{
					WorldAssetPackage = LevelInstanceInterface->GetWorldAssetPackage();

					World = OuterActor->GetWorld();

					// Handle Blueprint Editing. In this case we are going to rely on the preview world to build the filters
					if (!World && OuterActor->IsTemplate() && OuterActor->GetClass()->ClassGeneratedBy)
					{
						if (FBlueprintEditor* BlueprintEditor = static_cast<FBlueprintEditor*>(GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(OuterActor->GetClass()->ClassGeneratedBy, false)))
						{
							if (FPreviewScene* PreviewScene = BlueprintEditor->GetPreviewScene())
							{
								World = PreviewScene->GetWorld();
								break;
							}
						}
					}
				}
				else if (WorldAssetPackage != LevelInstanceInterface->GetWorldAssetPackage())
				{
					// Multiple WorldAssetPackages not supported
					return nullptr;
				}
			}
		}
	}

	// No matching WorldAssetPackage for selected Level Instances
	// or Actor is a template and doesn't have world
	if (WorldAssetPackage.IsEmpty() || !World)
	{
		return nullptr;
	}

	UWorldPartitionSubsystem* WorldPartitionSubsystem = World->GetSubsystem<UWorldPartitionSubsystem>();

	// Get the Default Filter for selected WorldAssetPackage
	TSharedPtr<FWorldPartitionActorFilter> Filter = MakeShared<FWorldPartitionActorFilter>(WorldPartitionSubsystem->GetWorldPartitionActorFilter(WorldAssetPackage, Types.GetValue()));

	// Set its name based on single/multi selection (root node name in outliner)
	Filter->DisplayName = LevelInstances.Num() == 1 ? ActorLabel : TEXT("(Multiple Actors)");

	// Gather Filters for selected level instances
	TArray<const FWorldPartitionActorFilter*> SelectedFilters;
	Algo::Transform(LevelInstances, SelectedFilters, [](const auto& Pair) { return &Pair.Key->GetFilter(); });

	// Create Mode Filter which holds the final values for the filter
	return MakeShared<FWorldPartitionActorFilterMode::FFilter>(Filter, SelectedFilters);
}

void FLevelInstanceFilterPropertyTypeCustomization::ApplyFilter(TSharedRef<IPropertyHandle> PropertyHandle, const FWorldPartitionActorFilterMode& Mode)
{
	const FScopedTransaction Transaction(LOCTEXT("WorldPartitionActorFilterApply_Transaction", "Apply Level Instance Filter"));
	for (auto& [LevelInstance, ObjectIndex] : LevelInstances)
	{
		ULevelInstanceComponent* Component = LevelInstance->GetLevelInstanceComponent();
		FWorldPartitionActorFilter ComponentFilter = Component->GetFilter();
		Mode.Apply(ComponentFilter);
		
		// Setting value through PropertyHandle will take care of propagating to instances if we are editing a template (Blueprint)
		PropertyHandle->SetPerObjectValue(ObjectIndex, ComponentFilter.ToString(), EPropertyValueSetFlags::DefaultFlags);
	}
}

#undef LOCTEXT_NAMESPACE