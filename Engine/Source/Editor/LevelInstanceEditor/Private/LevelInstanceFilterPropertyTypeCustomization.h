// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "Internationalization/Text.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "Templates/SharedPointer.h"
#include "WorldPartition/Filter/WorldPartitionActorFilterPropertyTypeCustomization.h"
#include "WorldPartition/Filter/WorldPartitionActorFilterMode.h"
#include "LevelInstance/LevelInstanceComponent.h"

class ILevelInstanceInterface;

class FLevelInstancePropertyTypeIdentifier : public IPropertyTypeIdentifier
{
public:
	FLevelInstancePropertyTypeIdentifier(bool bInIsEditFilter)
		: bIsEditFilter(bInIsEditFilter)
	{}

	virtual bool IsPropertyTypeCustomized(const IPropertyHandle& PropertyHandle) const override
	{
		return bIsEditFilter ? PropertyHandle.HasMetaData(TEXT("LevelInstanceEditFilter")) : PropertyHandle.HasMetaData(TEXT("LevelInstanceFilter"));
	}

private:
	bool bIsEditFilter;
};

// Registered (FLevelInstanceEditorModule::StartupModule) Property Customization for properties of type FWorldPartitionActorFilter for Level Instances
struct FLevelInstanceFilterPropertyTypeCustomization : public FWorldPartitionActorFilterPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance(bool bInIsEditFilter)
	{
		return MakeShareable(new FLevelInstanceFilterPropertyTypeCustomization(bInIsEditFilter));
	}
private:
	FLevelInstanceFilterPropertyTypeCustomization(bool bInIsEditFilter)
		: bIsEditFilter(bInIsEditFilter)
	{}

	virtual TSharedPtr<FWorldPartitionActorFilterMode::FFilter> CreateModeFilter(TArray<UObject*> OuterObjects) override;
	virtual void ApplyFilter(TSharedRef<IPropertyHandle> PropertyHandle, const FWorldPartitionActorFilterMode& Mode) override;

	TMap<ILevelInstanceInterface*, int32> LevelInstances;
	bool bIsEditFilter;
};
