// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelInstancePivotDetails.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "DetailWidgetRow.h"
#include "Engine/Level.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "SEnumCombo.h"
#include "ScopedTransaction.h"
#include "Engine/Brush.h"
#include "Engine/World.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "LevelInstance/LevelInstanceEditorPivotInterface.h"
#include "LevelInstance/LevelInstanceEditorPivot.h"
#include "SLevelInstancePivotPicker.h"

#define LOCTEXT_NAMESPACE "FLevelInstancePivotDetails"

FLevelInstancePivotDetails::FLevelInstancePivotDetails()
{
}

TSharedRef<IDetailCustomization> FLevelInstancePivotDetails::MakeInstance()
{
	return MakeShareable(new FLevelInstancePivotDetails);
}

void FLevelInstancePivotDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> EditingObjects;
	DetailBuilder.GetObjectsBeingCustomized(EditingObjects);

	if (EditingObjects.Num() > 1)
	{
		return;
	}

	TWeakObjectPtr<AActor> EditingObject = Cast<AActor>(EditingObjects[0].Get());

	UWorld* World = EditingObject->GetWorld();

	if (!World)
	{
		return;
	}

	PivotActor = nullptr;
	PivotType = ELevelInstancePivotType::CenterMinZ;

	IDetailCategoryBuilder& ChangePivotCategory = DetailBuilder.EditCategory("Change Pivot", FText::GetEmpty(), ECategoryPriority::Important);
	
	ChangePivotCategory.AddCustomRow(LOCTEXT("PivotTypeRow", "Type"), false)
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("PivotTypeText", "Type"))
			.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
		]
		.ValueContent()
		[
			SNew(SEnumComboBox, StaticEnum<ELevelInstancePivotType>())
			.ContentPadding(2.f)
			.CurrentValue(this, &FLevelInstancePivotDetails::GetSelectedPivotType)
			.OnEnumSelectionChanged(this, &FLevelInstancePivotDetails::OnSelectedPivotTypeChanged, EditingObject)
			.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
		];

	ChangePivotCategory.AddCustomRow(LOCTEXT("PivotActorRow", "Actor"), false)
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("PivotActorText", "Actor"))
			.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
		]
		.ValueContent()
		[
			SNew(SLevelInstancePivotPicker)
			.IsEnabled(this, &FLevelInstancePivotDetails::IsPivotActorSelectionEnabled)
			.OnPivotActorPicked(this, &FLevelInstancePivotDetails::OnPivotActorPicked, EditingObject)
		];
			
	ChangePivotCategory.AddCustomRow(LOCTEXT("ApplyRow", "Apply"), false)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1)
			[
				SNew(SButton)
				.IsEnabled(this, &FLevelInstancePivotDetails::IsApplyButtonEnabled, EditingObject)
				.HAlign(HAlign_Center)
				.OnClicked(this, &FLevelInstancePivotDetails::OnApplyButtonClicked, EditingObject)
				[
					SNew(STextBlock)
					.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
					.Text(LOCTEXT("ChangePivotButton", "Apply"))
				]
			]
		];
}

bool FLevelInstancePivotDetails::IsApplyButtonEnabled(TWeakObjectPtr<AActor> LevelInstancePivot) const
{
	return PivotType != ELevelInstancePivotType::Actor || PivotActor.IsValid();
}

FReply FLevelInstancePivotDetails::OnApplyButtonClicked(TWeakObjectPtr<AActor> LevelInstancePivot)
{
	if (LevelInstancePivot.IsValid())
	{
		FScopedTransaction Transaction(LOCTEXT("ChangePivot", "Change Level Instance Pivot"));
		FLevelInstanceEditorPivotHelper::SetPivot(CastChecked<ILevelInstanceEditorPivotInterface>(LevelInstancePivot.Get()), PivotType, PivotActor.Get());
	}
	return FReply::Handled();
}

int32 FLevelInstancePivotDetails::GetSelectedPivotType() const
{
	return (int32)PivotType;
}

void FLevelInstancePivotDetails::OnSelectedPivotTypeChanged(int32 NewValue, ESelectInfo::Type SelectionType, TWeakObjectPtr<AActor> LevelInstancePivot)
{
	PivotType = (ELevelInstancePivotType)NewValue;

	ShowPivotLocation(LevelInstancePivot);
}

bool FLevelInstancePivotDetails::IsPivotActorSelectionEnabled() const
{
	return PivotType == ELevelInstancePivotType::Actor;
}

void FLevelInstancePivotDetails::OnPivotActorPicked(AActor* InPivotActor, TWeakObjectPtr<AActor> LevelInstancePivot)
{
	PivotActor = InPivotActor;

	ShowPivotLocation(LevelInstancePivot);
}

void FLevelInstancePivotDetails::ShowPivotLocation(const TWeakObjectPtr<AActor>& LevelInstancePivot)
{
	// Avoid showing invalid location
	if (PivotType == ELevelInstancePivotType::Actor && PivotActor == nullptr)
	{
		return;
	}

	FVector PivotLocation = FLevelInstanceEditorPivotHelper::GetPivot(CastChecked<ILevelInstanceEditorPivotInterface>(LevelInstancePivot.Get()), PivotType, PivotActor.Get());
	FLevelInstanceEditorPivotHelper::ShowPivotLocation(PivotLocation);
}


#undef LOCTEXT_NAMESPACE