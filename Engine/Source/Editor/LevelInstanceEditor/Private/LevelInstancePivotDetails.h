// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Types/SlateEnums.h"
#include "IDetailCustomization.h"
#include "LevelInstance/LevelInstanceTypes.h"
#include "Input/Reply.h"

class AActor;

class FLevelInstancePivotDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	/** End IDetailCustomization interface */

private:
	/** Use MakeInstance to create an instance of this class */
	FLevelInstancePivotDetails();

	bool IsApplyButtonEnabled(TWeakObjectPtr<AActor> LevelInstancePivot) const;
	FReply OnApplyButtonClicked(TWeakObjectPtr<AActor> LevelInstancePivot);

	int32 GetSelectedPivotType() const;
	void OnSelectedPivotTypeChanged(int32 NewValue, ESelectInfo::Type SelectionType, TWeakObjectPtr<AActor> LevelInstancePivot);

	void OnPivotActorPicked(AActor* InPivotActor, TWeakObjectPtr<AActor> LevelInstancePivot);
	bool IsPivotActorSelectionEnabled() const;

	void ShowPivotLocation(const TWeakObjectPtr<AActor>& LevelInstancePivot);

	// Settings for Apply button
	ELevelInstancePivotType PivotType;
	TWeakObjectPtr<AActor> PivotActor;
};