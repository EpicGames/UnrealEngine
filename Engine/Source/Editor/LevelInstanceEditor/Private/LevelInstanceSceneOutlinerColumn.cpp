// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelInstanceSceneOutlinerColumn.h"
#include "Styling/SlateIconFinder.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Images/SImage.h"
#include "GameFramework/Actor.h"
#include "LevelInstance/LevelInstanceActor.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "ActorTreeItem.h"
#include "SceneOutlinerHelpers.h"

#define LOCTEXT_NAMESPACE "LevelInstanceColumn"

namespace LevelInstanceColumnPrivate
{
	FName Name("Level Instance");

	const FText ToolTipIsOverriden = LOCTEXT("IsOverridenTooltip", "This actor is overridden.");
	const FText ToolTipIsOverridenAndContainsOverrides = LOCTEXT("IsOverridenAndContainsOverridesTooltip", "This level instance is overridden, and so is at least one of its children.");
	const FText ToolTipContainsOverrides = LOCTEXT("ContainsOverridesTooltip", "At least one child of this level instance is overridden.");

	void GetBrushesAndToolTipForItem(const TWeakObjectPtr<AActor> Actor, const FSlateBrush*& OutHasOVerrideBrush, const FSlateBrush*& OutContainsOverrideBrush, const FText*& OutToolTipText)
	{
		OutToolTipText = nullptr;
		OutHasOVerrideBrush = nullptr;
		OutContainsOverrideBrush = nullptr;

		if (!Actor.IsValid())
		{
			return;
		}

		if (const ILevelInstanceInterface* LevelInstance = Cast<ILevelInstanceInterface>(Actor); LevelInstance && LevelInstance->GetPropertyOverrideAsset() && !LevelInstance->IsEditing())
		{
			const bool bIsEditable = !Actor->IsInLevelInstance() || Actor->IsInEditLevelInstance();

			OutContainsOverrideBrush = bIsEditable ? FAppStyle::GetBrush("LevelInstance.ColumnOverrideContainerInsideEditable") : FAppStyle::GetBrush("LevelInstance.ColumnOverrideContainerInside");

			if (Actor->HasLevelInstancePropertyOverrides())
			{
				OutHasOVerrideBrush = Actor->HasEditableLevelLevelInstancePropertyOverrides() ? FAppStyle::GetBrush("LevelInstance.ColumnOverrideContainerHereEditable") : FAppStyle::GetBrush("LevelInstance.ColumnOverrideContainerHere");
				OutToolTipText = &ToolTipIsOverridenAndContainsOverrides;
			}
			else
			{
				OutHasOVerrideBrush = bIsEditable ? FAppStyle::GetBrush("LevelInstance.ColumnOverrideContainerEditable") : FAppStyle::GetBrush("LevelInstance.ColumnOverrideContainer");
				OutToolTipText = &ToolTipContainsOverrides;
			}
		}
		else if (Actor->HasLevelInstancePropertyOverrides())
		{
			OutHasOVerrideBrush = Actor->HasEditableLevelLevelInstancePropertyOverrides() ? FAppStyle::GetBrush("LevelInstance.ColumnOverrideHereEditable") : FAppStyle::GetBrush("LevelInstance.ColumnOverrideHere");
			OutToolTipText = &ToolTipIsOverriden;
		}
	}
}

FName FLevelInstanceSceneOutlinerColumn::GetID()
{
	return LevelInstanceColumnPrivate::Name;
}

SHeaderRow::FColumn::FArguments FLevelInstanceSceneOutlinerColumn::ConstructHeaderRowColumn()
{
	return SHeaderRow::Column(GetColumnID())
		.FixedWidth(24.f)
		.HAlignHeader(HAlign_Center)
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Center)
		.VAlignCell(VAlign_Center)
		.DefaultTooltip(FText::FromName(GetColumnID()))
		[
			SNew(SImage)
			.ColorAndOpacity(FSlateColor::UseForeground())
			.Image(FSlateIconFinder::FindIconBrushForClass(ALevelInstance::StaticClass()))
		];
}

const TSharedRef<SWidget> FLevelInstanceSceneOutlinerColumn::ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row)
{
	const TWeakObjectPtr<AActor> Actor = SceneOutliner::FSceneOutlinerHelpers::GetActorFromOutlinerTreeItem(*TreeItem, WeakOutliner.Pin());

	if (!Actor.IsValid())
	{
		return SNullWidget::NullWidget;
	}
	
	// First overlay slot is optional
	return SNew(SOverlay)
			+SOverlay::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image_Lambda([Actor]()
				{
					const FSlateBrush* OutHasOVerrideBrush;
					const FSlateBrush* OutContainsOverrideBrush;
					const FText* OutToolTipText;
					LevelInstanceColumnPrivate::GetBrushesAndToolTipForItem(Actor, OutHasOVerrideBrush, OutContainsOverrideBrush, OutToolTipText);
					return OutContainsOverrideBrush;
				})
			]
			+SOverlay::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image_Lambda([Actor]()
				{
					const FSlateBrush* OutHasOVerrideBrush;
					const FSlateBrush* OutContainsOverrideBrush;
					const FText* OutToolTipText;
					LevelInstanceColumnPrivate::GetBrushesAndToolTipForItem(Actor, OutHasOVerrideBrush, OutContainsOverrideBrush, OutToolTipText);
					return OutHasOVerrideBrush;
				})
				.ToolTipText_Lambda([Actor]()
				{
					const FSlateBrush* OutHasOVerrideBrush;
					const FSlateBrush* OutContainsOverrideBrush;
					const FText* OutToolTipText;
					LevelInstanceColumnPrivate::GetBrushesAndToolTipForItem(Actor, OutHasOVerrideBrush, OutContainsOverrideBrush, OutToolTipText);
					return OutToolTipText ? *OutToolTipText : FText();
				})
			];
}

#undef LOCTEXT_NAMESPACE
