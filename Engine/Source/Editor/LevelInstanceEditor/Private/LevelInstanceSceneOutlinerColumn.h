// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SceneOutlinerFwd.h"
#include "ISceneOutliner.h"
#include "ISceneOutlinerColumn.h"
#include "Widgets/Views/SHeaderRow.h"

template<typename ItemType> class STableRow;
class FLevelInstanceSceneOutlinerColumn : public ISceneOutlinerColumn
{
public:
	FLevelInstanceSceneOutlinerColumn(ISceneOutliner& SceneOutliner)
		: WeakOutliner(StaticCastSharedRef<ISceneOutliner>(SceneOutliner.AsShared())) {}
	virtual ~FLevelInstanceSceneOutlinerColumn() = default;
	static FName GetID();

	//~ Begin ISceneOutlinerColumn Interface
	virtual FName GetColumnID() override { return GetID(); }
	virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn() override;
	virtual const TSharedRef<SWidget> ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row) override;
	virtual bool SupportsSorting() const override { return false; }
	//~ End ISceneOutlinerColumn Interface

private:
	TWeakPtr<ISceneOutliner> WeakOutliner;
};
