// Copyright Epic Games, Inc. All Rights Reserved.
#if WITH_EDITOR

#include "PackedLevelActorUtils.h"
#include "PackedLevelActor/PackedLevelActor.h"
#include "PackedLevelActor/PackedLevelActorBuilder.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Engine/World.h"
#include "Engine/Blueprint.h"
#include "EngineUtils.h"
#include "Misc/ScopedSlowTask.h"
#include "Editor.h"
#include "UObject/UObjectIterator.h"

#define LOCTEXT_NAMESPACE "FPackedLevelActorUtils"

void FPackedLevelActorUtils::GetPackedBlueprintsForWorldAsset(const TSoftObjectPtr<UWorld>& InWorldAsset, TSet<TSoftObjectPtr<UBlueprint>>& OutPackedBlueprintAssets, bool bInLoadedOnly)
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(FName("AssetRegistry"));
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

	if (InWorldAsset.IsNull())
	{
		return;
	}
			
	TArray<FSoftObjectPath> AssetsToProcess;
	AssetsToProcess.Add(InWorldAsset.ToSoftObjectPath());
	TSet<FSoftObjectPath> ProcessedAssets;

	while (!AssetsToProcess.IsEmpty())
	{
		FSoftObjectPath ProcessingAsset = AssetsToProcess.Pop();
		ProcessedAssets.Add(ProcessingAsset);

		TArray<FName> ReferencerPackages;
		AssetRegistry.GetReferencers(*ProcessingAsset.GetLongPackageName(), ReferencerPackages);

		for (FName ReferencerPackage : ReferencerPackages)
		{
			TArray<FAssetData> Assets;
			AssetRegistry.GetAssetsByPackageName(ReferencerPackage, Assets);

			for (const FAssetData& Asset : Assets)
			{
				if (UClass* AssetClass = Asset.GetClass(bInLoadedOnly ? EResolveClass::No : EResolveClass::Yes))
				{
					if (TSubclassOf<UBlueprint> BlueprintClass = AssetClass)
					{
						if (UBlueprint* Blueprint = Cast<UBlueprint>(Asset.GetAsset()); Blueprint && Blueprint->GeneratedClass && Blueprint->GeneratedClass->IsChildOf<APackedLevelActor>())
						{
							FSoftObjectPath BlueprintAsset(Blueprint);
							if (!ProcessedAssets.Contains(BlueprintAsset) && ProcessedAssets.Contains(Blueprint->GeneratedClass->GetDefaultObject<APackedLevelActor>()->GetWorldAsset().ToSoftObjectPath()))
							{
								OutPackedBlueprintAssets.Add(Blueprint);
								AssetsToProcess.Add(BlueprintAsset);
							}						
						}
					}
					else if (TSubclassOf<AActor> ActorClass = AssetClass; ActorClass && ActorClass->ImplementsInterface(ULevelInstanceInterface::StaticClass()))
					{
						FSoftObjectPath ReferencingAsset(Asset.GetSoftObjectPath().GetWithoutSubPath());
						if (!ProcessedAssets.Contains(ReferencingAsset))
						{
							AssetsToProcess.Add(ReferencingAsset);
						}
					}
					else if (TSubclassOf<UWorld> WorldClass = AssetClass)
					{
						FSoftObjectPath ReferencingAsset(Asset.GetSoftObjectPath());
						if (!ProcessedAssets.Contains(ReferencingAsset))
						{
							AssetsToProcess.Add(ReferencingAsset);
						}
					}
				}
			}
		}
	}
}

bool FPackedLevelActorUtils::CanPack()
{
	if (!GEditor)
	{
		return false;
	}

	if (GEditor->GetPIEWorldContext())
	{
		return false;
	}

	if (ULevelInstanceSubsystem* LevelInstanceSubsystem = GEditor->GetEditorWorldContext().World()->GetSubsystem<ULevelInstanceSubsystem>())
	{
		return !LevelInstanceSubsystem->GetEditingLevelInstance();
	}

	return false;
}

void FPackedLevelActorUtils::PackAllLoadedActors()
{
	if (!CanPack())
	{
		return;
	}
	
	TSet<APackedLevelActor*> PackedLevelActorsToUpdate;
	TSet<UBlueprint*> BlueprintsToUpdate;

	if (UWorld* EditorWorld = GEditor->GetEditorWorldContext().World())
	{
		for (TActorIterator<APackedLevelActor> PackedLevelActorIt(EditorWorld); PackedLevelActorIt; ++PackedLevelActorIt)
		{
			APackedLevelActor* PackedLevelActor = *PackedLevelActorIt;
			UBlueprint* Blueprint = PackedLevelActor->GetRootBlueprint();
			if (Blueprint)
			{
				BlueprintsToUpdate.Add(Blueprint);
			}
			else
			{
				PackedLevelActorsToUpdate.Add(PackedLevelActor);
			}
		}
	}
	

	int32 Count = BlueprintsToUpdate.Num() + PackedLevelActorsToUpdate.Num();
	if (!Count)
	{
		return;
	}

	GEditor->SelectNone(true, true);

	FScopedSlowTask SlowTask(Count, (LOCTEXT("TaskPackLevels", "Packing Levels")));
	SlowTask.MakeDialog();

	auto UpdateProgress = [&SlowTask]()
	{
		if (SlowTask.CompletedWork < SlowTask.TotalAmountOfWork)
		{
			SlowTask.EnterProgressFrame(1, FText::Format(LOCTEXT("TaskPackLevelProgress", "Packing Level {0} of {1}"), FText::AsNumber(SlowTask.CompletedWork), FText::AsNumber(SlowTask.TotalAmountOfWork)));
		}
	};

	TSharedPtr<FPackedLevelActorBuilder> Builder = FPackedLevelActorBuilder::CreateDefaultBuilder();
	const bool bCheckoutAndSave = false;
	for (UBlueprint* Blueprint : BlueprintsToUpdate)
	{
		Builder->UpdateBlueprint(Blueprint, bCheckoutAndSave);
		UpdateProgress();
	}

	for (APackedLevelActor* PackedLevelActor : PackedLevelActorsToUpdate)
	{
		PackedLevelActor->UpdateLevelInstanceFromWorldAsset();
		UpdateProgress();
	}
}

bool FPackedLevelActorUtils::CreateOrUpdateBlueprint(ILevelInstanceInterface* InLevelInstance, TSoftObjectPtr<UBlueprint> InBlueprintAsset, bool bCheckoutAndSave, bool bPromptForSave)
{
	return FPackedLevelActorBuilder::CreateDefaultBuilder()->CreateOrUpdateBlueprint(InLevelInstance, InBlueprintAsset, bCheckoutAndSave, bPromptForSave);
}

bool FPackedLevelActorUtils::CreateOrUpdateBlueprint(TSoftObjectPtr<UWorld> InWorldAsset, TSoftObjectPtr<UBlueprint> InBlueprintAsset, bool bCheckoutAndSave, bool bPromptForSave)
{
	return FPackedLevelActorBuilder::CreateDefaultBuilder()->CreateOrUpdateBlueprint(InWorldAsset, InBlueprintAsset, bCheckoutAndSave, bPromptForSave);
}

void FPackedLevelActorUtils::UpdateBlueprint(UBlueprint* InBlueprint, bool bCheckoutAndSave)
{
	FPackedLevelActorBuilder::CreateDefaultBuilder()->UpdateBlueprint(InBlueprint, bCheckoutAndSave);
}

void FPackedLevelActorUtils::UpdateAllPackedBlueprintsForWorldAssetBlueprint(const TSoftObjectPtr<UWorld>& InWorldAsset, bool bInLoadedOnly)
{
	FScopedSlowTask SlowTask(0.0f, LOCTEXT("UpdatePackedBlueprintsProgress", "Updating Packed Blueprints..."));
	TSet<TSoftObjectPtr<UBlueprint>> BlueprintAssets;
	FPackedLevelActorUtils::GetPackedBlueprintsForWorldAsset(InWorldAsset, BlueprintAssets, bInLoadedOnly);
	TSharedPtr<FPackedLevelActorBuilder> Builder = FPackedLevelActorBuilder::CreateDefaultBuilder();
	for (TSoftObjectPtr<UBlueprint> BlueprintAsset : BlueprintAssets)
	{
		if (UBlueprint* Blueprint = BlueprintAsset.Get())
		{
			Builder->UpdateBlueprint(Blueprint, false);
		}
	}
}

#undef LOCTEXT_NAMESPACE

#endif