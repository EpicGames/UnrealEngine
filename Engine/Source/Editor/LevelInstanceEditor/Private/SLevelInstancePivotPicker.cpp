// Copyright Epic Games, Inc. All Rights Reserved.
#include "SLevelInstancePivotPicker.h"
#include "Widgets/Text/STextBlock.h"
#include "Modules/ModuleManager.h"
#include "SceneOutlinerPublicTypes.h"
#include "SceneOutlinerModule.h"
#include "ISceneOutlinerTreeItem.h"
#include "ActorMode.h"
#include "ActorPickingMode.h"
#include "ActorTreeItem.h"

#define LOCTEXT_NAMESPACE "SLevelInstancePivotPicker"

void SLevelInstancePivotPicker::Construct(const FArguments& InArgs)
{
	OnPivotActorPickedEvent = InArgs._OnPivotActorPicked;

	FOnSceneOutlinerItemPicked OnItemPicked = FOnSceneOutlinerItemPicked::CreateLambda([this](TSharedRef<ISceneOutlinerTreeItem> Item)
	{
		if (FActorTreeItem* ActorItem = Item->CastTo<FActorTreeItem>())
		{
			if (ActorItem->IsValid())
			{
				OnPivotActorPicked(ActorItem->Actor.Get());
			}
		}
	});

	FCreateSceneOutlinerMode ModeFactory = FCreateSceneOutlinerMode::CreateLambda([OnItemPicked](SSceneOutliner* Outliner)
	{
		FActorModeParams Params;
		Params.SceneOutliner = Outliner;
		Params.bHideComponents = true;
		Params.bHideLevelInstanceHierarchy = false;
		Params.bHideUnloadedActors = true;
		Params.bHideEmptyFolders = true;
		Params.bCanInteractWithSelectableActorsOnly = false;
		return new FActorPickingMode(Params, OnItemPicked);
	});

	FSceneOutlinerInitializationOptions InitOptions;
	{
		InitOptions.bShowHeaderRow = false;
		InitOptions.bFocusSearchBoxWhenOpened = true;
		InitOptions.bShowTransient = true;
		InitOptions.ModeFactory = ModeFactory;
	}

	// Actor selector to allow the user to choose a parent actor
	FSceneOutlinerModule& SceneOutlinerModule = FModuleManager::LoadModuleChecked<FSceneOutlinerModule>("SceneOutliner");

	this->ChildSlot
	[
		SAssignNew(PivotActorPicker, SComboButton)
		.ContentPadding(2.f)
		.HasDownArrow(false)
		.ButtonContent()
		[
			SNew(STextBlock)
			.Text(this, &SLevelInstancePivotPicker::GetSelectedPivotActorText)
			.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
		]
		.MenuContent()
		[
			SNew(SBox)
			.MaxDesiredHeight(400.0f)
			.WidthOverride(300.0f)
			[
				SceneOutlinerModule.CreateActorPicker(InitOptions, FOnActorPicked())
			]
		]
	];
}

FText SLevelInstancePivotPicker::GetSelectedPivotActorText() const
{
	return PivotActor.IsValid() ? FText::FromString(PivotActor->GetActorLabel()) : LOCTEXT("none", "None");
}

void SLevelInstancePivotPicker::OnPivotActorPicked(AActor* PickedActor)
{
	PivotActor = PickedActor;
	PivotActorPicker->SetIsOpen(false);
			
	OnPivotActorPickedEvent.ExecuteIfBound(PickedActor);
}

#undef LOCTEXT_NAMESPACE