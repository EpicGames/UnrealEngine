// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SComboButton.h"

//////////////////////////////////////////////////////////////////////
class AActor;

DECLARE_DELEGATE_OneParam(FOnPivotActorPicked, AActor*)

class SLevelInstancePivotPicker : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SLevelInstancePivotPicker) {}
		SLATE_EVENT(FOnPivotActorPicked, OnPivotActorPicked)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);

private:
	FText GetSelectedPivotActorText() const;
	void OnPivotActorPicked(AActor* PickedActor);

	FOnPivotActorPicked OnPivotActorPickedEvent;
	TWeakObjectPtr<AActor> PivotActor;
	TSharedPtr<SComboButton> PivotActorPicker;
};