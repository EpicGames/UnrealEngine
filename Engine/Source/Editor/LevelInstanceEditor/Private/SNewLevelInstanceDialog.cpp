// Copyright Epic Games, Inc. All Rights Reserved.
#include "SNewLevelInstanceDialog.h"

#include "Containers/BitArray.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "DetailsViewArgs.h"
#include "Fonts/SlateFontInfo.h"
#include "GameFramework/Actor.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "IDetailPropertyRow.h"
#include "IDetailsView.h"
#include "IStructureDetailsView.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Optional.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorDelegates.h"
#include "PropertyEditorModule.h"
#include "Serialization/Archive.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateTypes.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Class.h"
#include "UObject/ObjectPtr.h"
#include "UObject/StructOnScope.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWidget.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"
#include "SLevelInstancePivotPicker.h"

class IPropertyHandle;

#define LOCTEXT_NAMESPACE "LevelInstanceEditor"

const FVector2D SNewLevelInstanceDialog::DEFAULT_WINDOW_SIZE = FVector2D(400, 250);

void SNewLevelInstanceDialog::Construct(const FArguments& InArgs)
{
	ParentWindowPtr = InArgs._ParentWindow.Get();
	bClickedOk = false;
		
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	TSharedPtr<IStructureDetailsView> StructureDetailsView;

	FDetailsViewArgs DetailsViewArgs;
	{
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.bHideSelectionTip = true;
		DetailsViewArgs.bLockable = false;
		DetailsViewArgs.bSearchInitialKeyFocus = true;
		DetailsViewArgs.bUpdatesFromSelection = false;
		DetailsViewArgs.NotifyHook = nullptr;
		DetailsViewArgs.bShowOptions = true;
		DetailsViewArgs.bShowModifiedPropertiesOption = false;
		DetailsViewArgs.bShowScrollBar = false;
	}

	FStructureDetailsViewArgs StructureViewArgs;
	{
		StructureViewArgs.bShowObjects = true;
		StructureViewArgs.bShowAssets = true;
		StructureViewArgs.bShowClasses = true;
		StructureViewArgs.bShowInterfaces = true;
	}

	StructureDetailsView = PropertyEditorModule.CreateStructureDetailView(DetailsViewArgs, StructureViewArgs, nullptr);
	StructureDetailsView->GetDetailsView()->SetGenericLayoutDetailsDelegate(FOnGetDetailCustomizationInstance::CreateStatic(&FNewLevelInstanceParamsDetails::MakeInstance, InArgs._PivotActors.Get()));

	FStructOnScope* Struct = new FStructOnScope(FNewLevelInstanceParams::StaticStruct(), (uint8*)&CreationParams);
	StructureDetailsView->SetStructureData(MakeShareable(Struct));

	this->ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					StructureDetailsView->GetWidget()->AsShared()
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				[
					SNew(SSpacer)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.f)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.IsEnabled(this, &SNewLevelInstanceDialog::IsOkEnabled)
					.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					.OnClicked(this, &SNewLevelInstanceDialog::OnOkClicked)
					.Text(LOCTEXT("OkButton", "Ok"))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.f)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					.OnClicked(this, &SNewLevelInstanceDialog::OnCancelClicked)
					.Text(LOCTEXT("CancelButton", "Cancel"))
				]
			]
			+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				[
					SNew(SCheckBox)
					.IsChecked_Lambda([this]() -> ECheckBoxState
					{
						return CreationParams.bAlwaysShowDialog ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					})
					.OnCheckStateChanged_Lambda([this](ECheckBoxState State)
					{
						CreationParams.bAlwaysShowDialog = State == ECheckBoxState::Checked;
					})
					[
						SNew(STextBlock)
						.Font(FCoreStyle::Get().GetFontStyle("SmallFont"))
						.Text(LOCTEXT("AlwaysShowDialog", "Always show dialog"))
						.ToolTipText(LOCTEXT("AlwaysShowDialogToolTip", "Show this dialog everytime a level instance is created. Can be changed in editor preferences (Content Editors > Level Instance)."))
					]
				]
			]
		]
	];
}

bool SNewLevelInstanceDialog::IsOkEnabled() const
{
	if (CreationParams.PivotType == ELevelInstancePivotType::Actor && !CreationParams.PivotActor)
	{
		return false;
	}

	return true;
}

FReply SNewLevelInstanceDialog::OnOkClicked()
{
	bClickedOk = true;
	ParentWindowPtr.Pin()->RequestDestroyWindow();
	return FReply::Handled();
}

FReply SNewLevelInstanceDialog::OnCancelClicked()
{
	bClickedOk = false;
	ParentWindowPtr.Pin()->RequestDestroyWindow();
	return FReply::Handled();
}

void FNewLevelInstanceParamsDetails::OnSelectedPivotActorChanged(AActor* NewValue)
{
	CreationParams->PivotActor = NewValue;
}

bool FNewLevelInstanceParamsDetails::IsPivotActorSelectionEnabled() const
{
	return CreationParams->PivotType == ELevelInstancePivotType::Actor;
}

void FNewLevelInstanceParamsDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	// Make sure we actually get a valid struct before continuing
	TArray<TSharedPtr<FStructOnScope>> Structs;
	DetailBuilder.GetStructsBeingCustomized(Structs);

	if (Structs.Num() == 0)
	{
		// Nothing being customized
		return;
	}

	const UStruct* Struct = Structs[0]->GetStruct();
	if (!Struct || Struct != FNewLevelInstanceParams::StaticStruct())
	{
		// Invalid struct
		return;
	}

	// Get ptr to our actual type
	CreationParams = (FNewLevelInstanceParams*)Structs[0]->GetStructMemory();

	TSharedPtr<IPropertyHandle> PivotTypeProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(FNewLevelInstanceParams, PivotType), (UClass*)FNewLevelInstanceParams::StaticStruct());
	TSharedPtr<IPropertyHandle> PivotActorProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(FNewLevelInstanceParams, PivotActor), (UClass*)FNewLevelInstanceParams::StaticStruct());


	IDetailCategoryBuilder& PivotCategoryBuilder = DetailBuilder.EditCategory("Pivot", FText::GetEmpty(), ECategoryPriority::Uncommon);
	PivotCategoryBuilder.AddProperty(PivotTypeProperty);
	IDetailPropertyRow& PivotActorPropertyRow = PivotCategoryBuilder.AddProperty(PivotActorProperty);
	
	TSharedPtr<SWidget> NameWidget;
	TSharedPtr<SWidget> ValueWidget;
	FDetailWidgetRow Row;
	PivotActorPropertyRow.GetDefaultWidgets(NameWidget, ValueWidget, Row);

	const bool bShowChildren = true;
	PivotActorPropertyRow.CustomWidget(bShowChildren)
		.NameContent()
		.MinDesiredWidth(Row.NameWidget.MinWidth)
		.MaxDesiredWidth(Row.NameWidget.MaxWidth)
		[
			NameWidget.ToSharedRef()
		]
		.ValueContent()
		.MinDesiredWidth(Row.ValueWidget.MinWidth)
		.MaxDesiredWidth(Row.ValueWidget.MaxWidth)
		[
			SNew(SLevelInstancePivotPicker)
			.OnPivotActorPicked(this, &FNewLevelInstanceParamsDetails::OnSelectedPivotActorChanged)
			.IsEnabled(this, &FNewLevelInstanceParamsDetails::IsPivotActorSelectionEnabled)
		];
}

#undef LOCTEXT_NAMESPACE
