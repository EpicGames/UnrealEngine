// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "IDetailCustomization.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "LevelInstance/LevelInstanceTypes.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class AActor;
class IDetailLayoutBuilder;
class SWidget;
class SWindow;

//////////////////////////////////////////////////////////////////////////
// SNewLevelInstanceDialog

class SNewLevelInstanceDialog : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SNewLevelInstanceDialog) {}
		/** A pointer to the parent window */
		SLATE_ATTRIBUTE(TSharedPtr<SWindow>, ParentWindow)
		SLATE_ATTRIBUTE(TArray<AActor*>, PivotActors)
	SLATE_END_ARGS()

	~SNewLevelInstanceDialog() = default;

	/**
	 * Construct this widget.  Called by the SNew() Slate macro.
	 *
	 * @param	InArgs				Declaration used by the SNew() macro to construct this widget
	 * @param	InViewModel			The UI logic not specific to slate
	 */
	void Construct(const FArguments& InArgs);

	static const FVector2D DEFAULT_WINDOW_SIZE;

	bool ClickedOk() const { return bClickedOk; }
	
	FNewLevelInstanceParams& GetCreationParams() { return CreationParams; }

private:
	FReply OnOkClicked();
	bool IsOkEnabled() const;

	FReply OnCancelClicked();

	/** Pointer to the parent window, so we know to destroy it when done */
	TWeakPtr<SWindow> ParentWindowPtr;

	/** Dialog Result */
	FNewLevelInstanceParams CreationParams;

	/** Dialog Result */
	bool bClickedOk;
};

class FNewLevelInstanceParamsDetails : public IDetailCustomization
{
public:

	FNewLevelInstanceParamsDetails(TArray<AActor*> InPivotActors)
		: CreationParams(nullptr), PivotActors(InPivotActors)
	{};

	static TSharedRef<IDetailCustomization> MakeInstance(TArray<AActor*> InPivotActors)
	{
		return MakeShareable(new FNewLevelInstanceParamsDetails(InPivotActors));
	}

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

	void OnSelectedPivotActorChanged(AActor* NewValue);
	bool IsPivotActorSelectionEnabled() const;

protected:
	FNewLevelInstanceParams* CreationParams;
	TArray<AActor*> PivotActors;
};
