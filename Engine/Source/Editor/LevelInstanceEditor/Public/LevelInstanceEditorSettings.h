// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Editor/UnrealEdEngine.h"
#include "Engine/DeveloperSettings.h"
#include "LevelInstance/LevelInstanceTypes.h"
#include "LevelInstanceEditorSettings.generated.h"

UCLASS(config = Editor, defaultconfig, meta = (DisplayName = "Level Instance"))
class ULevelInstanceEditorSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	ULevelInstanceEditorSettings();

	/** List of info for all known LevelInstance template maps */
	UPROPERTY(config)
	TArray<FTemplateMapInfo> TemplateMapInfos;

	UPROPERTY(config)
	FString LevelInstanceClassName;

	UPROPERTY(config, EditAnywhere, Category="World Partition", meta = (ToolTip="Create World Partition Level Instances with Streaming Enabled/Disabled by default"))
	bool bEnableStreaming;

	UPROPERTY(config, EditAnywhere, Category="World Partition", meta = (ToolTip="Allow Editing Level Instances with Streaming Enabled"))
	bool bIsEditInPlaceStreamingEnabled;
};

UCLASS(config = EditorPerProjectUserSettings, meta = (DisplayName = "Level Instance"))
class ULevelInstanceEditorPerProjectUserSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	ULevelInstanceEditorPerProjectUserSettings();

	static void UpdateFrom(const FNewLevelInstanceParams& Params);

	/** Gets the category for the settings, some high level grouping like, Editor, Engine, Game...etc. */
	virtual FName GetCategoryName() const override { return TEXT("ContentEditors"); }
		
	/** If false, create dialog will not be shown and last settings will be used. */
	UPROPERTY(config, EditAnywhere, Category = Create)
	bool bAlwaysShowDialog;

	UPROPERTY(config, EditAnywhere, Category = Pivot)
	ELevelInstancePivotType PivotType;

	/** If true, allows read only selection of actors inside a Level Instance */
	UPROPERTY(config, EditAnywhere, Category = "Selection", meta = (ToolTip = "Allow Read Only Selection of Level Instance Level Actors"))
	bool bIsSubSelectionEnabled;

	/** If true, allows read only selection of actors inside a Level Instance through the viewport */
	UPROPERTY(config, EditAnywhere, Category = "Selection", meta = (DisplayName = "Is Viewport Sub Selection Enabled (Shift + Mouse Scroll Up/Down)", ToolTip = "Allow Read Only Selection of Level Instance Level Actors in the Viewport (Shortcut: Shift + Mouse Scroll Up/Down)"))
	bool bIsViewportSubSelectionEnabled;

	/**
	 * When the Level Instance is broken via "Level->Break..", its actors will be placed inside the folder the LI is
	 * inside of, under a subfolder with the name of the Level Instance, and also keeping their original folder structure.
	 * So if i.e. the Level Instance Actor is called "Desert/LI_House2", and an actor inside is named "Lights/Light_Sun",
	 * the actor will be moved to "Desert/LI_House2/Lights/Light_Sun" in the outer level.
	 *
	 * If this flag is not set, actors will be placed either in the root folder of the outer level (but their original
	 * folders from the LI kept), or, if "Current Folder" is set, they'll be moved there without any subfolders.
	 */
	UPROPERTY(config, EditAnywhere, Category = Break)
	bool bKeepFoldersDuringBreak = false;
};
