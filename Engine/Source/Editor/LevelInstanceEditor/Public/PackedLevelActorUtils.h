// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_EDITOR

#include "UObject/SoftObjectPtr.h"

class UWorld;
class UBlueprint;
class ILevelInstanceInterface;

class FPackedLevelActorUtils
{
public:
	static void GetPackedBlueprintsForWorldAsset(const TSoftObjectPtr<UWorld>& InWorldAsset, TSet<TSoftObjectPtr<UBlueprint>>& OutPackedBlueprintAssets, bool bInLoadedOnly = false);

	static bool CanPack();

	static void PackAllLoadedActors();

	static bool CreateOrUpdateBlueprint(ILevelInstanceInterface* InLevelInstance, TSoftObjectPtr<UBlueprint> InBlueprintAsset, bool bCheckoutAndSave = true, bool bPromptForSave = true);

	static bool CreateOrUpdateBlueprint(TSoftObjectPtr<UWorld> InWorldAsset, TSoftObjectPtr<UBlueprint> InBlueprintAsset, bool bCheckoutAndSave = true, bool bPromptForSave = true);
	
	static void UpdateBlueprint(UBlueprint* InBlueprint, bool bCheckoutAndSave = true);

	static void UpdateAllPackedBlueprintsForWorldAssetBlueprint(const TSoftObjectPtr<UWorld>& InWorldAsset, bool bInLoadedOnly = false);
};

#endif 